// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class AudioSynesthesia : ModuleRules
	{
        public AudioSynesthesia(ReadOnlyTargetRules Target) : base(Target)
		{
            PublicDependencyModuleNames.AddRange(
				new string[] {
                    "Core",
					"CoreUObject",
					"Engine",
					"SignalProcessing",
					"AudioMixerCore",
					"AudioMixer",
                    "AudioAnalyzer",
					"AudioSynesthesiaCore"
                }
            );
		}
	}
}
