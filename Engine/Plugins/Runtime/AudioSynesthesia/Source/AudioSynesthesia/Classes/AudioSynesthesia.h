// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioAnalyzer.h"
#include "AudioSynesthesia.generated.h"

/** UAudioSynesthesiaSettings
 *
 * Defines asset actions for derived UAudioSynthesiaSettings subclasses.
 */
UCLASS(Abstract, Blueprintable, MinimalAPI)
class UAudioSynesthesiaSettings : public UAudioAnalyzerSettings
{
	GENERATED_BODY()

public:

#if WITH_EDITOR
	AUDIOSYNESTHESIA_API const TArray<FText>& GetAssetActionSubmenus() const;
	AUDIOSYNESTHESIA_API FColor GetTypeColor() const override;
#endif
};