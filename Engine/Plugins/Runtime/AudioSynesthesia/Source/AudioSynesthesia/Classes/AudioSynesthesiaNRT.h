// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioAnalyzerNRT.h"
#include "AudioSynesthesiaNRT.generated.h"

/** UAudioSynesthesiaNRTSettings
 *
 * Defines asset actions for derived UAudioSynthesiaNRTSettings subclasses.
 */
UCLASS(Abstract, Blueprintable, MinimalApi)
class UAudioSynesthesiaNRTSettings : public UAudioAnalyzerNRTSettings
{
	GENERATED_BODY()

	public:

		AUDIOSYNESTHESIA_API const TArray<FText>& GetAssetActionSubmenus() const;

#if WITH_EDITOR
		AUDIOSYNESTHESIA_API FColor GetTypeColor() const override;
#endif
};

/** UAudioSynesthesiaNRT
 *
 * Defines asset actions for derived UAudioSynthesiaNRT subclasses.
 */
UCLASS(Abstract, Blueprintable, MinimalApi)
class UAudioSynesthesiaNRT : public UAudioAnalyzerNRT
{
	GENERATED_BODY()

	public:

		AUDIOSYNESTHESIA_API const TArray<FText>& GetAssetActionSubmenus() const;

#if WITH_EDITOR
		AUDIOSYNESTHESIA_API FColor GetTypeColor() const override;
#endif
};

