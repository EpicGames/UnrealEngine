// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioSynesthesia.h"
#include "ConstantQNRT.h"
#include "Sound/SoundSubmix.h"
#include "ConstantQ.generated.h"

#define UE_API AUDIOSYNESTHESIA_API

/** UConstantQSettings
 *
 * Settings for a UConstantQ analyzer.
 */
UCLASS(MinimalAPI, Blueprintable)
class UConstantQSettings : public UAudioSynesthesiaSettings
{
	GENERATED_BODY()
public:

	UConstantQSettings() {}

	/** Starting frequency for first bin of CQT */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, DisplayName = "Starting Frequency (Hz)", meta = (ClampMin = "20.0", ClampMax = "20000"))
	float StartingFrequencyHz = 40.0f;

	/** Total number of resulting constant Q bands. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "1", ClampMax = "96"))
	int32 NumBands = 48;

	/** Number of bands within an octave. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "1", ClampMax = "24"))
	float NumBandsPerOctave = 12.0f;

	/** Number of seconds between cqt measurements */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, DisplayName = "Analysis Period (s)", meta = (ClampMin = "0.01", ClampMax = "1.0"))
	float AnalysisPeriodInSeconds = 0.01f;

	/** If true, multichannel audio is downmixed to mono with equal amplitude scaling. If false, each channel gets it's own CQT result. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer)
	bool bDownmixToMono = false;

	/** Size of FFT. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=AudioAnalyzer)
	EConstantQFFTSizeEnum FFTSize = EConstantQFFTSizeEnum::XLarge;

	/** Type of window to be applied to input audio */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=AudioAnalyzer)
	EFFTWindowType WindowType = EFFTWindowType::Blackman;

	/** Type of spectrum to use. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=AudioAnalyzer)
	EAudioSpectrumType SpectrumType = EAudioSpectrumType::PowerSpectrum;
		
	/** Stretching factor to control overlap of adjacent bands. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=AudioAnalyzer, meta = (ClampMin = "0.01", ClampMax = "2.0"))
	float BandWidthStretch = 1.0f;
		
	/** Normalization scheme used to generate band windows. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=AudioAnalyzer)
	EConstantQNormalizationEnum CQTNormalization = EConstantQNormalizationEnum::EqualEnergy;

	/** Noise floor to use when normalizing CQT */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=AudioAnalyzer, meta = (DisplayName = "Noise Floor (dB)", ClampMin = "-120.0", ClampMax = "0.0"))
	float NoiseFloorDb = -60.0f;

	/** Convert UConstantQSettings to FConstantQSettings */
	UE_API TUniquePtr<Audio::IAnalyzerSettings> GetSettings(const float InSampleRate, const int32 InNumChannels) const;

#if WITH_EDITOR
	UE_API virtual FText GetAssetActionName() const override;

	UE_API virtual UClass* GetSupportedClass() const override;
#endif // WITH_EDITOR
};

/** The results of the ConstantQ analysis. */
USTRUCT(BlueprintType)
struct FConstantQResults
{
	GENERATED_USTRUCT_BODY()

	// The time in seconds since analysis began of this ConstantQ analysis result
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float TimeSeconds = 0.0f;

	// The spectrum values from the FFT
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	TArray<float> SpectrumValues;
};

/** Delegate to receive all ConstantQ results per channel (time-stamped in an array) since last delegate call. If bDownmixToMono setting is true, results will be in channel index 0. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnConstantQResults, int32, ChannelIndex, const TArray<FConstantQResults>&, ConstantQResults);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnConstantQResultsNative, UConstantQAnalyzer*, int32, const TArray<FConstantQResults>&);

/** Delegate to receive only the most recent overall ConstantQ result per channel. If bDownmixToMono setting is true, results will be in channel index 0.*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnLatestConstantQResults, int32, ChannelIndex, const FConstantQResults&, LatestConstantQResults);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnLatestConstantQResultsNative, UConstantQAnalyzer*, int32, const FConstantQResults&);

/** UConstantQAnalyzer
 *
 * UConstantQAnalyzer calculates the temporal evolution of constant q transform for a given
 * audio bus in real-time. ConstantQ is available for individual channels or the overall audio bus.
 */
UCLASS(MinimalAPI, Blueprintable)
class UConstantQAnalyzer : public UAudioAnalyzer
{
	GENERATED_BODY()
public:
	UE_API UConstantQAnalyzer();

	/** The settings for the audio analyzer.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AudioAnalyzer)
	TObjectPtr<UConstantQSettings> Settings;

	/** Delegate to receive all Spectrum results, per-channel, since last delegate call. If bDownmixToMono setting is true, results will be in channel index 0. */
	UPROPERTY(BlueprintAssignable)
	FOnConstantQResults OnConstantQResults;

	FOnConstantQResultsNative OnConstantQResultsNative;

	/** Delegate to receive the latest per-channel Spectrum results. If bDownmixToMono setting is true, results will be in channel index 0. */
	UPROPERTY(BlueprintAssignable)
	FOnLatestConstantQResults OnLatestConstantQResults;

	FOnLatestConstantQResultsNative OnLatestConstantQResultsNative;

	/** Convert UConstantQSettings to FConstantQSettings */
	UE_API TUniquePtr<Audio::IAnalyzerSettings> GetSettings(const int32 InSampleRate, const int32 InNumChannels) const override;

	/** Broadcasts results to any delegates if hooked up. */
	UE_API void BroadcastResults() override;

	UFUNCTION(BlueprintCallable, Category = "Audio Analyzer")
	UE_API void GetCenterFrequencies(UPARAM(DisplayName = "Center Frequencies") TArray<float>& OutCenterFrequencies);

	UFUNCTION(BlueprintCallable, Category = "Audio Analyzer")
	UE_API UPARAM(DisplayName = "Num Center Frequencies") const int32 GetNumCenterFrequencies() const;

protected:

	/** Return the name of the IAudioAnalyzerFactory associated with this UAudioAnalyzer */
	UE_API FName GetAnalyzerFactoryName() const override;
};

#undef UE_API
