// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioSynesthesiaNRT.h"
#include "ConstantQNRT.generated.h"

#define UE_API AUDIOSYNESTHESIA_API

enum class EAudioSpectrumType : uint8;
enum class EFFTWindowType : uint8;

/** EConstantQNormalizationEnum
 *
 * Enumeration of available normalization schemes for ConstantQ frequency domain windows.
 */
UENUM(BlueprintType)
enum class EConstantQNormalizationEnum : uint8
{
	/** Normalize bands by euclidean norm. Good when using magnitude spectrum. */
	EqualEuclideanNorm	UMETA(DisplayName="Equal Euclidean Norm"),

	/** Normalize bands by energy. Good when using power spectrum. */
	EqualEnergy			UMETA(DisplayName="Equal Energy"),

	/** Normalize bands by their maximum values. Will result in relatively strong high frequences because the upper constant Q bands have larger bandwidths. */
	EqualAmplitude 		UMETA(DisplayName="Equal Amplitude"),
};

/** EContantQFFTSizeEnum
 *
 *  Enumeration of available FFT sizes in audio frames.
 */
UENUM(BlueprintType)
enum class EConstantQFFTSizeEnum : uint8
{
	// 64
	Min UMETA(DisplayName = "64"),
	
	// 128
	XXSmall UMETA(DisplayName = "128"),

	// 256
	XSmall UMETA(DisplayName = "256"),

	// 512
	Small UMETA(DisplayName = "512"),

	// 1024
	Medium UMETA(DisplayName = "1024"),

	// 2048
	Large UMETA(DisplayName = "2048"),

	// 4096
	XLarge UMETA(DisplayName = "4096"),

	// 8192
	XXLarge UMETA(DisplayName = "8192"),

	// 16384
	Max UMETA(DisplayName = "16384")
};
										 
/** UConstantQNRTSettings
 *
 * Settings for a UConstantQNRT analyzer.
 */
UCLASS(MinimalAPI, Blueprintable)
class UConstantQNRTSettings : public UAudioSynesthesiaNRTSettings
{
	GENERATED_BODY()
	public:

		UE_API UConstantQNRTSettings();

		/** Starting frequency for first bin of CQT */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "20.0", ClampMax = "20000"))
		float StartingFrequency;

		/** Total number of resulting constant Q bands. */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "1", ClampMax = "96"))
		int32 NumBands;

		/** Number of bands within an octave. */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "1", ClampMax = "24"))
		float NumBandsPerOctave;
		/** Number of seconds between cqt measurements */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "0.01", ClampMax = "1.0"))
		float AnalysisPeriod;

		/** If true, multichannel audio is downmixed to mono with equal amplitude scaling. If false, each channel gets it's own CQT result. */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer)
		bool bDownmixToMono;

		/** Size of FFT. */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=AudioAnalyzer)
		EConstantQFFTSizeEnum FFTSize;

		/** Type of window to be applied to input audio */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=AudioAnalyzer)
		EFFTWindowType WindowType;

		/** Type of spectrum to use. */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=AudioAnalyzer)
		EAudioSpectrumType SpectrumType;
		
		/** Stretching factor to control overlap of adjacent bands. */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=AudioAnalyzer, meta = (ClampMin = "0.01", ClampMax = "2.0"))
		float BandWidthStretch;
		
		/** Normalization scheme used to generate band windows. */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=AudioAnalyzer)
		EConstantQNormalizationEnum CQTNormalization;

		/** Noise floor to use when normalizing CQT */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = AudioAnalyzer, meta = (DisplayName = "Noise Floor (dB)", ClampMin = "-120.0", ClampMax = "0.0"))
		float NoiseFloorDb;

		/** Convert UConstantQNRTSettings to FConstantQNRTSettings */
		UE_API TUniquePtr<Audio::IAnalyzerNRTSettings> GetSettings(const float InSampleRate, const int32 InNumChannels) const;

#if WITH_EDITOR
		UE_API virtual FText GetAssetActionName() const override;

		UE_API virtual UClass* GetSupportedClass() const override;
#endif
};


/** UConstantQNRT
 *
 * UConstantQNRT calculates the temporal evolution of constant q transform for a given 
 * sound. ConstantQ is available for individual channels or the overall sound asset.
 */
UCLASS(MinimalAPI, Blueprintable)
class UConstantQNRT : public UAudioSynesthesiaNRT
{
	GENERATED_BODY()

	public:

		UE_API UConstantQNRT();

		/** The settings for the audio analyzer.  */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer)
		TObjectPtr<UConstantQNRTSettings> Settings;

		/** Get a specific channel cqt of the analyzed sound at a given time. */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API void GetChannelConstantQAtTime(const float InSeconds, const int32 InChannel, TArray<float>& OutConstantQ) const;

		/** Get a specific channel cqt of the analyzed sound at a given time. */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API void GetNormalizedChannelConstantQAtTime(const float InSeconds, const int32 InChannel, TArray<float>& OutConstantQ) const;

		/** Convert ULoudnessNRTSettings to FLoudnessNRTSettings */
 		UE_API virtual TUniquePtr<Audio::IAnalyzerNRTSettings> GetSettings(const float InSampleRate, const int32 InNumChannels) const override;

#if WITH_EDITOR
		UE_API virtual FText GetAssetActionName() const override;

		UE_API virtual UClass* GetSupportedClass() const override;

		UE_API virtual bool ShouldEventTriggerAnalysis(FPropertyChangedEvent& PropertyChangeEvent) override;
#endif // WITH_EDITOR

	protected:

		/** Return the name of the IAudioAnalyzerNRTFactory associated with this UAudioAnalyzerNRT */
		UE_API virtual FName GetAnalyzerNRTFactoryName() const override;

};

#undef UE_API
