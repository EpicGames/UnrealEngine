// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioSynesthesia.h"
#include "LKFSFactory.h"

#include "LKFS.generated.h"

#define UE_API AUDIOSYNESTHESIA_API

/** ULKFSSettings
 *
 * Settings for a ULKFSAnalyzer.
 */
UCLASS(MinimalAPI, Blueprintable)
class ULKFSSettings : public UAudioSynesthesiaSettings
{
	GENERATED_BODY()
public:

	ULKFSSettings() {}

	/** Number of seconds between loudness measurements. The standard is 0.1 seconds. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer, meta = (ClampMin = "0.01", ClampMax = "1.00"))
	float AnalysisPeriod = 0.1f;

	/** Number of seconds of audio analyzed for each loudness measurements. The standard is 0.4 seconds. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "0.10", ClampMax = "1.00"))
	float AnalysisWindowDuration = 0.4f;

	/** Duration of audio analyzed for short term loudness. The standard is 3.0 seconds. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "1.00", ClampMax = "30.00"))
	float ShortTermLoudnessDuration = Audio::FLKFSSettings::StandardShortTermLoudnessDuration;

	/** Number of seconds between long term (integrated and gated) loudness updates.  The standard is 1.0 seconds. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "0.1", ClampMax = "10.00"))
	float IntegratedLoudnessAnalysisPeriod = Audio::FLKFSSettings::StandardIntegratedLoudnessAnalysisPeriod;

	/** Duration of audio analyzed for long term (integrated and gated) loudness. The real time LKFS analyzer
	 * performs integrated loudness duration over a large window instead of performing calculations over
	 * the entire audio stream. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "10.00", ClampMax = "3600.00"))
	float IntegratedLoudnessDuration = Audio::FLKFSSettings::DefaultIntegratedLoudnessDuration;

	/** Should the Loudness Range (LRA) statistic be calculated for each channel? */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	bool bCalculatePerChannelLoudnessRange = false;

	/** Should the overall Loudness Range (LRA) statistic be calculated? */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	bool bCalculateOverallLoudnessRange = false;

	/** Convert ULKFSSettings to IAnalyzerSettings */
	UE_API TUniquePtr<Audio::IAnalyzerSettings> GetSettings(const int32 InSampleRate, const int32 InNumChannels) const;

#if WITH_EDITOR
	UE_API virtual FText GetAssetActionName() const override;

	UE_API virtual UClass* GetSupportedClass() const override;
#endif
};

/** Delegate to receive all overall loudness results (time-stamped in an array) since last delegate call. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnOverallLKFSResults, const TArray<FLKFSResults>&, LKFSResults);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnOverallLKFSResultsNative, ULKFSAnalyzer*, TArrayView<const FLKFSResults>);

/** Delegate to receive only the most recent overall meter results. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnLatestOverallLKFSResults, const FLKFSResults&, LatestOverallLKFSResults);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLatestOverallLKFSResultsNative, ULKFSAnalyzer*, const FLKFSResults&);

/** Delegate to receive all meter results per channel (time-stamped in an array) since last delegate call. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnPerChannelLKFSResults, int32, ChannelIndex, const TArray<FLKFSResults>&, LKFSResults);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnPerChannelLKFSResultsNative, ULKFSAnalyzer*, int32, TArrayView<const FLKFSResults>);

/** Delegate to receive only the most recent overall meter result per channel. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnLatestPerChannelLKFSResults, int32, ChannelIndex, const FLKFSResults&, LatestLKFSResults);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnLatestPerChannelLKFSResultsNative, ULKFSAnalyzer*, int32, const FLKFSResults&);

/** ULKFSAnalyzer
 *
 * ULKFSAnalyzer calculates the current amplitude of an
 * audio bus in real-time.
 */
UCLASS(MinimalAPI, Blueprintable)
class ULKFSAnalyzer : public UAudioAnalyzer
{
	GENERATED_BODY()

public:

	UE_API ULKFSAnalyzer();

	/** The settings for the meter audio analyzer.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AudioAnalyzer)
	TObjectPtr<ULKFSSettings> Settings;

	/** Delegate to receive all overall meter results since last delegate call. */
	UPROPERTY(BlueprintAssignable)
	FOnOverallLKFSResults OnOverallLKFSResults;

	FOnOverallLKFSResultsNative OnOverallLKFSResultsNative;

	/** Delegate to receive all meter results, per-channel, since last delegate call. */
	UPROPERTY(BlueprintAssignable)
	FOnPerChannelLKFSResults OnPerChannelLKFSResults;

	FOnPerChannelLKFSResultsNative OnPerChannelLKFSResultsNative;

	/** Delegate to receive the latest overall meter results. */
	UPROPERTY(BlueprintAssignable)
	FOnLatestOverallLKFSResults OnLatestOverallLKFSResults;

	FOnLatestOverallLKFSResultsNative OnLatestOverallLKFSResultsNative;

	/** Delegate to receive the latest per-channel meter results. */
	UPROPERTY(BlueprintAssignable)
	FOnLatestPerChannelLKFSResults OnLatestPerChannelLKFSResults;

	FOnLatestPerChannelLKFSResultsNative OnLatestPerChannelLKFSResultsNative;

	/** Convert ULKFSSettings to IAnalyzerSettings */
	UE_API TUniquePtr<Audio::IAnalyzerSettings> GetSettings(const int32 InSampleRate, const int32 InNumChannels) const override;

	/** Broadcasts results to any delegates if hooked up. */
	UE_API void BroadcastResults() override;

protected:

	/** Return the name of the IAudioAnalyzerFactory associated with this UAudioAnalyzer */
	UE_API FName GetAnalyzerFactoryName() const override;
};

#undef UE_API
