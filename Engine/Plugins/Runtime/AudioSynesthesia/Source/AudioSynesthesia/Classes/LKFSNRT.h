// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioSynesthesiaNRT.h"
#include "LKFSNRTFactory.h"
#include "LKFSNRT.generated.h"

#define UE_API AUDIOSYNESTHESIA_API


/** ULKFSNRTSettings
 *
 * Settings for a ULKFSNRT analyzer.
 */
UCLASS(MinimalAPI, Blueprintable)
class ULKFSNRTSettings : public UAudioSynesthesiaNRTSettings
{
	GENERATED_BODY()
	public:

		UE_API ULKFSNRTSettings();

		/** Number of seconds between loudness measurements */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "0.01", ClampMax = "1.00"))
		float AnalysisPeriod = 0.1f;

		/** Number of seconds of audio analyzed for a single window of measurement*/
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "0.10", ClampMax = "1.00"))
		float AnalysisWindowDuration = 0.4f;

		/** Number of seconds of audio analyzed when determining short term loudness.*/
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "1.00", ClampMax = "30.00"))
		float ShortTermLoudnessDuration = Audio::FLKFSNRTSettings::DefaultShortTermLoudnessDuration;

		/** Convert ULKFSNRTSettings to FLKFSNRTSettings */
		UE_API virtual TUniquePtr<Audio::IAnalyzerNRTSettings> GetSettings(const float InSampleRate, const int32 InNumChannels) const;

#if WITH_EDITOR
		UE_API virtual FText GetAssetActionName() const override;

		UE_API virtual UClass* GetSupportedClass() const override;
#endif
};



/** ULKFSNRT
 *
 * ULKFSNRT calculates the temporal evolution of perceptual loudness for a given 
 * sound. 
 */
UCLASS(MinimalAPI, Blueprintable)
class ULKFSNRT : public UAudioSynesthesiaNRT
{
	GENERATED_BODY()

	public:

		UE_API ULKFSNRT();

		/** The settings for the audio analyzer.  */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ExposeOnSpawn))
		TObjectPtr<ULKFSNRTSettings> Settings;

		/** Get the overall loudness of the analyzed sound at a given time. */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API void GetLoudnessAtTime(const float InSeconds, float& OutLoudness) const;

		/** Get a specific channel loudness of the analyzed sound at a given time. */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API void GetChannelLoudnessAtTime(const float InSeconds, const int32 InChannel, float& OutLoudness) const;

		/** Return the entire array of loudness data for a specific audio channel. */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API TArray<FLKFSNRTResults> GetLoudnessDataForChannel(int32 InChannel) const;

		/** Return the entire array of loudness data for the combination of all audio channels. */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API TArray<FLKFSNRTResults> GetLoudnessData() const;

		/** Return the instantaneous loudness data for a specific time and audio channel. */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API FLKFSNRTResults GetLoudnessDataForChannelAtTime(const float InSeconds, int32 InChannel) const;

		/** Return the instantaneous loudness data for the combination of all audio channels at a specific time. */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API FLKFSNRTResults GetLoudnessDataAtTime(const float InSeconds) const;

		/** Return the integrated (average) loudness for a specific audio channel. */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API float GetIntegratedLoudnessForChannel(int32 InChannel) const;

		/** Return the integrated (average) loudness for the combination of all audio channels. */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API float GetIntegratedLoudness() const;

		/** Return the gated loudness for a specific audio channel. */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API float GetGatedLoudnessForChannel(int32 InChannel) const;

		/** Return the gated loudness for the combination of all audio channels. */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API float GetGatedLoudness() const;

		/** Convert ULKFSNRTSettings to FLKFSNRTSettings */
 		UE_API TUniquePtr<Audio::IAnalyzerNRTSettings> GetSettings(const float InSampleRate, const int32 InNumChannels) const override;

#if WITH_EDITOR
		UE_API virtual FText GetAssetActionName() const override;

		UE_API virtual UClass* GetSupportedClass() const override;

		UE_API virtual bool ShouldEventTriggerAnalysis(FPropertyChangedEvent& PropertyChangeEvent) override;
#endif // WITH_EDITOR

	protected:

		/** Return the name of the IAudioAnalyzerNRTFactory associated with this UAudioAnalyzerNRT */
		UE_API FName GetAnalyzerNRTFactoryName() const override;

};

#undef UE_API
