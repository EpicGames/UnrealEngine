// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioSynesthesia.h"
#include "Loudness.generated.h"

#define UE_API AUDIOSYNESTHESIA_API

/** ELoudnessCurveTypeEnum
 *
 * Enumeration of available equal loudness curves. Loudness curves based on IEC 61672-1:2013 Electroacoustics - Sound level meters - Part 1: Specifications.
 */
UENUM(BlueprintType)
enum class ELoudnessCurveTypeEnum : uint8
{
	/** Loudness Curve A Weighting. Commonly used in environmental sound measurements. Best for low level sounds. */
	A 		UMETA(DisplayName = "A"),
	/** Loudness Curve B Weighting. Relative to "A", gives more precedence to frequencies below 1kHz. */
	B		UMETA(DisplayName = "B"),
	/** Loudness Curve C Weighting. Relative to "A" and "B", gives more precedence to frequencies below 1kHz. */
	C		UMETA(DisplayName = "C"),
	/** Loudness Curve D Weighting. Similar to "B" but with an emphasis on presence in the 2kHz-6KHz frequency range. */
	D		UMETA(DisplayName = "D"),
	/** Loudness Curve K Weighting. Correlates with subjective perception of recorded broadcast content. */
	K		UMETA(DisplayName = "K"),
	/** No loudness curve weighting. */
	None	UMETA(DisplayName = "None"),
};


/** ULoudnessSettings
 *
 * Settings for a ULoudness analyzer.
 */
UCLASS(MinimalAPI, Blueprintable)
class ULoudnessSettings : public UAudioSynesthesiaSettings
{
	GENERATED_BODY()
public:

	ULoudnessSettings() {}

	/** Number of seconds between loudness measurements */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer, meta = (ClampMin = "0.01", ClampMax = "0.25"))
	float AnalysisPeriod = 0.01f;

	/** Minimum analysis frequency for calculating loudness. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer, meta = (ClampMin = "20.0", ClampMax = "20000"))
	float MinimumFrequency = 20.0f;

	/** Maximum analysis frequency for calculating loudness. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer, meta = (ClampMin = "20.0", ClampMax = "20000"))
	float MaximumFrequency = 20000.0f;

	/** Type of equal loudness curve used in calculations */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	ELoudnessCurveTypeEnum CurveType = ELoudnessCurveTypeEnum::D;

	/** dB level to denote silence.  Used when calculating normalized loudness. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = AudioAnalyzer, meta = (ClampMin = "-100.0", ClampMax = "0"))
	float NoiseFloorDb = -60.0f;

	/** dB level to denote silence.  Used when calculating normalized loudness. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = AudioAnalyzer, meta = (ClampMin = "-100.0", ClampMax = "0"))
	float ExpectedMaxLoudness = 0.0f;

	/** Convert ULoudnessSettings to FLoudnessSettings */
	UE_API TUniquePtr<Audio::IAnalyzerSettings> GetSettings(const int32 InSampleRate, const int32 InNumChannels) const;

#if WITH_EDITOR
	UE_API virtual FText GetAssetActionName() const override;

	UE_API virtual UClass* GetSupportedClass() const override;
#endif
};

/** The results of the loudness analyis. */
USTRUCT(BlueprintType)
struct FLoudnessResults
{
	GENERATED_USTRUCT_BODY()
	
	// The raw loudness value in dB
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float Loudness = 0.0f;

	// The normalized loudness
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float NormalizedLoudness = 0.0f;

	// The normalized loudness
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float PerceptualEnergy = 0.0f;

	// The time in seconds since analysis began
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float TimeSeconds = 0.0f;
};

/** Delegate to receive all overall loudness results (time-stamped in an array) since last delegate call. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnOverallLoudnessResults, const TArray<FLoudnessResults>&, OverallLoudnessResults);

/** Delegate to receive only the most recent overall loudness result. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnLatestOverallLoudnessResults, const FLoudnessResults&, LatestOverallLoudnessResults);

/** Delegate to receive all loudness results per channel (time-stamped in an array) since last delegate call. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnPerChannelLoudnessResults, int32, ChannelIndex, const TArray<FLoudnessResults>&, LoudnessResults);

/** Delegate to receive only the most recent overall loudness result per channel. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnLatestPerChannelLoudnessResults, int32, ChannelIndex, const FLoudnessResults&, LatestLoudnessResults);

/** ULoudnessAnalyzer
 *
 * ULoudnessAnalyzer calculates the temporal evolution of perceptual loudness for a given
 * audio bus in real-time. Loudness is available for individual channels or the overall audio bus. Normalized
 * loudness values convert the range to 0.0 to 1.0 where 0.0 is the noise floor and 1.0 is the
 * maximum loudness of the particular sound.
 */
UCLASS(MinimalAPI, Blueprintable)
class ULoudnessAnalyzer : public UAudioAnalyzer
{
	GENERATED_BODY()

public:

	UE_API ULoudnessAnalyzer();

	/** The settings for the audio analyzer.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AudioAnalyzer, meta = (ExposeOnSpawn))
	TObjectPtr<ULoudnessSettings> Settings;

	/** Delegate to receive all overall loudness results since last delegate call. */
	UPROPERTY(BlueprintAssignable, Category = AudioAnalyzer)
	FOnOverallLoudnessResults OnOverallLoudnessResults;

	/** Delegate to receive all loudness results, per-channel, since last delegate call. */
	UPROPERTY(BlueprintAssignable, Category = AudioAnalyzer)
	FOnPerChannelLoudnessResults OnPerChannelLoudnessResults;

	/** Delegate to receive the latest overall loudness results. */
	UPROPERTY(BlueprintAssignable, Category = AudioAnalyzer)
	FOnLatestOverallLoudnessResults OnLatestOverallLoudnessResults;

	/** Delegate to receive the latest per-channel loudness results. */
	UPROPERTY(BlueprintAssignable, Category = AudioAnalyzer)
	FOnLatestPerChannelLoudnessResults OnLatestPerChannelLoudnessResults;

	/** Convert ULoudnessSettings to FLoudnessSettings */
	UE_API TUniquePtr<Audio::IAnalyzerSettings> GetSettings(const int32 InSampleRate, const int32 InNumChannels) const override;

	/** Broadcasts results to any delegates if hooked up. */
	UE_API void BroadcastResults() override;

protected:

	/** Return the name of the IAudioAnalyzerFactory associated with this UAudioAnalyzer */
	UE_API FName GetAnalyzerFactoryName() const override;
};

#undef UE_API
