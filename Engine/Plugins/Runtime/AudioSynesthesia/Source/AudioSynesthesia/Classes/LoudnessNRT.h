// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioSynesthesiaNRT.h"
#include "LoudnessNRT.generated.h"

#define UE_API AUDIOSYNESTHESIA_API

/** ELoudnessNRTCurveTypeEnum
 *
 * Enumeration of available equal loudness curves. Loudness curves based on IEC 61672-1:2013 Electroacoustics - Sound level meters - Part 1: Specifications.
 */
UENUM(BlueprintType)
enum class ELoudnessNRTCurveTypeEnum : uint8
{
	/** Loudness Curve A Weighting. Commonly used in environmental sound measurements. Best for low level sounds. */
	A 		UMETA(DisplayName="A"),
	/** Loudness Curve B Weighting. Relative to "A", gives more precedence to frequencies below 1kHz. */
	B		UMETA(DisplayName="B"),
	/** Loudness Curve C Weighting. Relative to "A" and "B", gives more precedence to frequencies below 1kHz. */
	C		UMETA(DisplayName="C"),
	/** Loudness Curve D Weighting. Similar to "B" but with an emphasis on presence in the 2kHz-6KHz frequency range. */
	D		UMETA(DisplayName="D"),
	/** Loudness Curve K Weighting. Correlates with subjective perception of recorded broadcast content. */
	K		UMETA(DisplayName = "K"),
	/** No loudness curve weighting. */
	None	UMETA(DisplayName="None"),
};

/** ULoudnessNRTSettings
 *
 * Settings for a ULoudnessNRT analyzer.
 */
UCLASS(MinimalAPI, Blueprintable)
class ULoudnessNRTSettings : public UAudioSynesthesiaNRTSettings
{
	GENERATED_BODY()
	public:

		UE_API ULoudnessNRTSettings();

		/** Number of seconds between loudness measurements */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "0.01", ClampMax = "0.25"))
		float AnalysisPeriod;

		/** Minimum analysis frequency for calculating loudness. */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "20.0", ClampMax = "20000"))
		float MinimumFrequency;

		/** Maximum analysis frequency for calculating loudness. */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "20.0", ClampMax = "20000"))
		float MaximumFrequency;

		/** Type of equal loudness curve used in calculations */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer)
		ELoudnessNRTCurveTypeEnum CurveType;

		/** dB level to denote silence.  Used when calculating normalized loudness. */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category=AudioAnalyzer, meta = (ClampMin = "-100.0", ClampMax="0"))
		float NoiseFloorDb;

		/** Convert ULoudnessNRTSettings to FLoudnessNRTSettings */
		UE_API TUniquePtr<Audio::IAnalyzerNRTSettings> GetSettings(const float InSampleRate, const int32 InNumChannels) const;

#if WITH_EDITOR
		UE_API virtual FText GetAssetActionName() const override;

		UE_API virtual UClass* GetSupportedClass() const override;
#endif
};


/** ULoudnessNRT
 *
 * ULoudnessNRT calculates the temporal evolution of perceptual loudness for a given 
 * sound. Loudness is available for individual channels or the overall sound asset. Normalized 
 * loudness values convert the range to 0.0 to 1.0 where 0.0 is the noise floor and 1.0 is the
 * maximum loudness of the particular sound.
 */
UCLASS(MinimalAPI, Blueprintable)
class ULoudnessNRT : public UAudioSynesthesiaNRT
{
	GENERATED_BODY()

	public:

		UE_API ULoudnessNRT();

		/** The settings for the audio analyzer.  */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ExposeOnSpawn))
		TObjectPtr<ULoudnessNRTSettings> Settings;

		/** Get the overall loudness of the analyzed sound at a given time. */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API void GetLoudnessAtTime(const float InSeconds, float& OutLoudness) const;

		/** Get a specific channel loudness of the analyzed sound at a given time. */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API void GetChannelLoudnessAtTime(const float InSeconds, const int32 InChannel, float& OutLoudness) const;

		/**
		 * Get the normalized overall loudness of the analyzed sound at a given time. Normalized loudness
		 * is always between 0.0 to 1.0. 0.0 refers to the noise floor while 1.0 refers to the maximum 
		 * loudness in the sound.
		 */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API void GetNormalizedLoudnessAtTime(const float InSeconds, float& OutLoudness) const;

		/**
		 * Get a specific channel normalized loudness of the analyzed sound at a given time. Normalized 
		 * loudness is always between 0.0 to 1.0. 0.0 refers to the noise floor while 1.0 refers to the 
		 * maximum loudness in the sound.
		 */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API void GetNormalizedChannelLoudnessAtTime(const float InSeconds, const int32 InChannel, float& OutLoudness) const;

		/** Convert ULoudnessNRTSettings to FLoudnessNRTSettings */
 		UE_API TUniquePtr<Audio::IAnalyzerNRTSettings> GetSettings(const float InSampleRate, const int32 InNumChannels) const override;

#if WITH_EDITOR
		UE_API virtual FText GetAssetActionName() const override;

		UE_API virtual UClass* GetSupportedClass() const override;

		UE_API virtual bool ShouldEventTriggerAnalysis(FPropertyChangedEvent& PropertyChangeEvent) override;
#endif // WITH_EDITOR

	protected:

		/** Return the name of the IAudioAnalyzerNRTFactory associated with this UAudioAnalyzerNRT */
		UE_API FName GetAnalyzerNRTFactoryName() const override;

};

#undef UE_API
