// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioSynesthesia.h"
#include "Meter.generated.h"

#define UE_API AUDIOSYNESTHESIA_API

class UMeterAnalyzer;

UENUM(BlueprintType)
enum class EMeterPeakType : uint8
{
	MeanSquared,
	RootMeanSquared,
	Peak,
	TruePeak,
	Count UMETA(Hidden)
};

/** UMeterSettings
 *
 * Settings for a UMeterAnalyzer.
 */
UCLASS(MinimalAPI, Blueprintable)
class UMeterSettings : public UAudioSynesthesiaSettings
{
	GENERATED_BODY()
public:

	UMeterSettings() {}

	/** Number of seconds between meter measurements */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer, meta = (ClampMin = "0.01", ClampMax = "0.25"))
	float AnalysisPeriod = 0.01f;

	/** Meter envelope type type */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	EMeterPeakType PeakMode = EMeterPeakType::RootMeanSquared;

	/** Meter attack time, in milliseconds */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer, meta = (ClampMin = "0"))
	int32 MeterAttackTime = 300;

	/** Meter release time, in milliseconds. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer, meta = (ClampMin = "0"))
	int32 MeterReleaseTime = 300;

	/** Peak hold time, in milliseconds */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer, meta = (ClampMin = "0"))
	int32 PeakHoldTime = 100;

	/** What volume threshold to throw clipping detection notifications. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer, meta = (ClampMin = "0.0"))
	float ClippingThreshold = 1.0f;

	/** Analog response. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	bool bIsAnalog = true;

	/** Convert UMeterSettings to IAnalyzerSettings */
	UE_API TUniquePtr<Audio::IAnalyzerSettings> GetSettings(const int32 InSampleRate, const int32 InNumChannels) const;

#if WITH_EDITOR
	UE_API virtual FText GetAssetActionName() const override;

	UE_API virtual UClass* GetSupportedClass() const override;
#endif
};

/** The results of the meter analysis. */
USTRUCT(BlueprintType)
struct FMeterResults
{
	GENERATED_USTRUCT_BODY()

	// The time in seconds since analysis began of this meter analysis result
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float TimeSeconds = 0.0f;

	// The meter value
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float MeterValue = 0.0f;

	// The peak value
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float PeakValue = 0.0f;

	// The number of samples in the period which were above the clipping threshold. Will be 0 if no clipping was detected.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	int32 NumSamplesClipping = 0;

	// The value (if non-zero) if clipping was detected above the clipping threshold
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float ClippingValue = 0.0f;
};

/** Delegate to receive all overall loudness results (time-stamped in an array) since last delegate call. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnOverallMeterResults, const TArray<FMeterResults>&, MeterResults);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnOverallMeterResultsNative, UMeterAnalyzer*, const TArray<FMeterResults>&);

/** Delegate to receive only the most recent overall meter results. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnLatestOverallMeterResults, const FMeterResults&, LatestOverallMeterResults);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLatestOverallMeterResultsNative, UMeterAnalyzer*, const FMeterResults&);

/** Delegate to receive all meter results per channel (time-stamped in an array) since last delegate call. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnPerChannelMeterResults, int32, ChannelIndex, const TArray<FMeterResults>&, MeterResults);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnPerChannelMeterResultsNative, UMeterAnalyzer*, int32, const TArray<FMeterResults>&);

/** Delegate to receive only the most recent overall meter result per channel. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnLatestPerChannelMeterResults, int32, ChannelIndex, const FMeterResults&, LatestMeterResults);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnLatestPerChannelMeterResultsNative, UMeterAnalyzer*, int32, const FMeterResults&);

/** UMeterAnalyzer
 *
 * UMeterAnalyzer calculates the current amplitude of an
 * audio bus in real-time.
 */
UCLASS(MinimalAPI, Blueprintable)
class UMeterAnalyzer : public UAudioAnalyzer
{
	GENERATED_BODY()

public:

	UE_API UMeterAnalyzer();

	/** The settings for the meter audio analyzer.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AudioAnalyzer)
	TObjectPtr<UMeterSettings> Settings;

	/** Delegate to receive all overall meter results since last delegate call. */
	UPROPERTY(BlueprintAssignable)
	FOnOverallMeterResults OnOverallMeterResults;

	FOnOverallMeterResultsNative OnOverallMeterResultsNative;

	/** Delegate to receive all meter results, per-channel, since last delegate call. */
	UPROPERTY(BlueprintAssignable)
	FOnPerChannelMeterResults OnPerChannelMeterResults;

	FOnPerChannelMeterResultsNative OnPerChannelMeterResultsNative;

	/** Delegate to receive the latest overall meter results. */
	UPROPERTY(BlueprintAssignable)
	FOnLatestOverallMeterResults OnLatestOverallMeterResults;

	FOnLatestOverallMeterResultsNative OnLatestOverallMeterResultsNative;

	/** Delegate to receive the latest per-channel meter results. */
	UPROPERTY(BlueprintAssignable)
	FOnLatestPerChannelMeterResults OnLatestPerChannelMeterResults;

	FOnLatestPerChannelMeterResultsNative OnLatestPerChannelMeterResultsNative;

	/** Convert UMeterSettings to IAnalyzerSettings */
	UE_API TUniquePtr<Audio::IAnalyzerSettings> GetSettings(const int32 InSampleRate, const int32 InNumChannels) const override;

	/** Broadcasts results to any delegates if hooked up. */
	UE_API void BroadcastResults() override;

protected:

	/** Return the name of the IAudioAnalyzerFactory associated with this UAudioAnalyzer */
	UE_API FName GetAnalyzerFactoryName() const override;
};

#undef UE_API
