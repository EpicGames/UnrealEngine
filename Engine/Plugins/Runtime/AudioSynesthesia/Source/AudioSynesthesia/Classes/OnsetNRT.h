// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioSynesthesiaNRT.h"
#include "OnsetNRT.generated.h"

#define UE_API AUDIOSYNESTHESIA_API
										 
/** UOnsetNRTSettings
 *
 * Settings for a UOnsetNRT analyzer.
 */
UCLASS(MinimalAPI, Blueprintable)
class UOnsetNRTSettings : public UAudioSynesthesiaNRTSettings
{
	GENERATED_BODY()
	public:

		UE_API UOnsetNRTSettings();

		/** If true, multichannel audio is downmixed to mono with equal amplitude scaling. If false, each channel gets it's own onset result. */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer)
		bool bDownmixToMono;

		/** Onset timestamp granularity onsets. Lower granularity takes longer to compute. */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "0.005", ClampMax = "0.25"))
		float GranularityInSeconds;

		/** Sensitivity of onset detector. Higher sensitivity will find more onsets. */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "0.0", ClampMax = "1.0"))
		float Sensitivity;

		/** Starting frequency for onset anlaysis. */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "20.0", ClampMax = "20000"))
		float MinimumFrequency;

		/** Starting frequency for onset anlaysis. */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer, meta = (ClampMin = "20.0", ClampMax = "20000"))
		float MaximumFrequency;
		
		/** Convert UOnsetNRTSettings to FOnsetNRTSettings */
		UE_API TUniquePtr<Audio::IAnalyzerNRTSettings> GetSettings(const float InSampleRate, const int32 InNumChannels) const;

#if WITH_EDITOR
		UE_API virtual FText GetAssetActionName() const override;

		UE_API virtual UClass* GetSupportedClass() const override;
#endif
};


/** UOnsetNRT
 *
 * UOnsetNRT calculates the temporal evolution of constant q transform for a given 
 * sound. Onset is available for individual channels or the overall sound asset.
 */
UCLASS(MinimalAPI, Blueprintable)
class UOnsetNRT : public UAudioSynesthesiaNRT
{
	GENERATED_BODY()

	public:

		UE_API UOnsetNRT();

		/** The settings for the audio analyzer.  */
		UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=AudioAnalyzer)
		TObjectPtr<UOnsetNRTSettings> Settings;

		/** Returns onsets which occured between start and end timestamps. */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API void GetChannelOnsetsBetweenTimes(const float InStartSeconds, const float InEndSeconds, const int32 InChannel, TArray<float>& OutOnsetTimestamps, TArray<float>& OutOnsetStrengths) const;

		/** Get a specific channel cqt of the analyzed sound at a given time. */
		UFUNCTION(BlueprintCallable, Category="Audio Analyzer")
		UE_API void GetNormalizedChannelOnsetsBetweenTimes(const float InStartSeconds, const float InEndSeconds, const int32 InChannel, TArray<float>& OutOnsetTimestamps, TArray<float>& OutOnsetStrengths) const;

		/** Convert ULoudnessNRTSettings to FLoudnessNRTSettings */
 		UE_API virtual TUniquePtr<Audio::IAnalyzerNRTSettings> GetSettings(const float InSampleRate, const int32 InNumChannels) const override;

#if WITH_EDITOR
		UE_API virtual FText GetAssetActionName() const override;

		UE_API virtual UClass* GetSupportedClass() const override;

		UE_API virtual bool ShouldEventTriggerAnalysis(FPropertyChangedEvent& PropertyChangeEvent) override;
#endif // WITH_EDITOR
	protected:

		/** Return the name of the IAudioAnalyzerNRTFactory associated with this UAudioAnalyzerNRT */
		UE_API virtual FName GetAnalyzerNRTFactoryName() const override;

};

#undef UE_API
