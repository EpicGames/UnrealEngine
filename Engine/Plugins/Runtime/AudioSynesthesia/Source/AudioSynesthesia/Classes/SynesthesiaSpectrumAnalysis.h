// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioSynesthesia.h"
#include "Sound/SoundSubmix.h"
#include "SynesthesiaSpectrumAnalysis.generated.h"

#define UE_API AUDIOSYNESTHESIA_API

class USynesthesiaSpectrumAnalyzer;

/** USynesthesiaSpectrumAnalysisSettings
 *
 * Settings for a USynesthesiaSpectrumAnalysisAnalyzer.
 */
UCLASS(MinimalAPI, Blueprintable)
class USynesthesiaSpectrumAnalysisSettings : public UAudioSynesthesiaSettings
{
	GENERATED_BODY()
public:

	USynesthesiaSpectrumAnalysisSettings() {}

	/** Number of seconds between SynesthesiaSpectrumAnalysis measurements */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer, meta = (ClampMin = "0.01", ClampMax = "0.25"))
	float AnalysisPeriod = 0.01f;

	/** Size of FFT. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	EFFTSize FFTSize = EFFTSize::DefaultSize;
	
	/** Type of spectrum to use. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	EAudioSpectrumType SpectrumType = EAudioSpectrumType::PowerSpectrum;

	/** Type of window to use. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	EFFTWindowType WindowType = EFFTWindowType::Hann;

	/** If true, multichannel audio is downmixed to mono with equal amplitude scaling. If false, each channel gets its own FFT result. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	bool bDownmixToMono = true;

	/** Convert USynesthesiaSpectrumAnalysisSettings to IAnalyzerSettings */
	UE_API TUniquePtr<Audio::IAnalyzerSettings> GetSettings(const int32 InSampleRate, const int32 InNumChannels) const;

#if WITH_EDITOR
	UE_API virtual FText GetAssetActionName() const override;

	UE_API virtual UClass* GetSupportedClass() const override;
#endif
};

/** The results of the spectrum analysis. */
USTRUCT(BlueprintType)
struct FSynesthesiaSpectrumResults
{
	GENERATED_USTRUCT_BODY()

	// The time in seconds since analysis began of this SynesthesiaSpectrumAnalysis analysis result
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float TimeSeconds = 0.0f;

	// The spectrum values from the FFT
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	TArray<float> SpectrumValues;
};

/** Delegate to receive all spectrum results per channel (time-stamped in an array) since last delegate call. If bDownmixToMono setting is true, results will be in channel index 0. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnSpectrumResults, int32, ChannelIndex, const TArray<FSynesthesiaSpectrumResults>&, SpectrumResults);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnSpectrumResultsNative, USynesthesiaSpectrumAnalyzer*, int32, const TArray<FSynesthesiaSpectrumResults>&);

/** Delegate to receive only the most recent overall spectrum result per channel. If bDownmixToMono setting is true, results will be in channel index 0.*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnLatestSpectrumResults, int32, ChannelIndex, const FSynesthesiaSpectrumResults&, LatestSpectrumResults);

/** shadow delegate declaration for above */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnLatestSpectrumResultsNative, USynesthesiaSpectrumAnalyzer*, int32, const FSynesthesiaSpectrumResults&);

/** USynesthesiaSpectrumAnalysisAnalyzer
 *
 * USynesthesiaSpectrumAnalysisAnalyzer calculates the current amplitude of an
 * audio bus in real-time.
 */
UCLASS(MinimalAPI, Blueprintable)
class USynesthesiaSpectrumAnalyzer : public UAudioAnalyzer
{
	GENERATED_BODY()

public:

	UE_API USynesthesiaSpectrumAnalyzer();

	/** The settings for the SynesthesiaSpectrumAnalysis audio analyzer.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AudioAnalyzer)
	TObjectPtr<USynesthesiaSpectrumAnalysisSettings> Settings;

	/** Delegate to receive all Spectrum results, per-channel, since last delegate call. If bDownmixToMono setting is true, results will be in channel index 0. */
	UPROPERTY(BlueprintAssignable)
	FOnSpectrumResults OnSpectrumResults;

	FOnSpectrumResultsNative OnSpectrumResultsNative;

	/** Delegate to receive the latest per-channel Spectrum results. If bDownmixToMono setting is true, results will be in channel index 0. */
	UPROPERTY(BlueprintAssignable)
	FOnLatestSpectrumResults OnLatestSpectrumResults;

	FOnLatestSpectrumResultsNative OnLatestSpectrumResultsNative;

	/** Convert USynesthesiaSpectrumAnalysisSettings to IAnalyzerSettings */
	UE_API TUniquePtr<Audio::IAnalyzerSettings> GetSettings(const int32 InSampleRate, const int32 InNumChannels) const override;

	/** Broadcasts results to any delegates if hooked up. */
	UE_API void BroadcastResults() override;

	UFUNCTION(BlueprintCallable, Category = "Audio Analyzer")
	UE_API void GetCenterFrequencies(UPARAM(DisplayName = "Sample Rate") const float InSampleRate, UPARAM(DisplayName = "Center Frequencies") TArray<float>& OutCenterFrequencies);

	UFUNCTION(BlueprintCallable, Category = "Audio Analyzer")
	UE_API UPARAM(DisplayName = "Num Center Frequencies") const int32 GetNumCenterFrequencies() const;

protected:

	/** Return the name of the IAudioAnalyzerFactory associated with this UAudioAnalyzer */
	UE_API FName GetAnalyzerFactoryName() const override;
};

#undef UE_API
