// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSynesthesia.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioSynesthesia)


#if WITH_EDITOR
const TArray<FText>& UAudioSynesthesiaSettings::GetAssetActionSubmenus() const
{
	static const TArray<FText> SubMenus
	{
		NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AssetSoundAnalysisSubmenu", "Analysis")
	};

	return SubMenus;
}

FColor UAudioSynesthesiaSettings::GetTypeColor() const
{
	return FColor(150.0f, 200.0f, 200.0f);
}
#endif
