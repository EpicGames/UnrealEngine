// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSynesthesiaModule.h"

#include "AudioAnalyzerModule.h"
#include "AudioSynesthesiaLog.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogAudioSynesthesia);

IMPLEMENT_MODULE(Audio::FAudioSynesthesiaModule, AudioSynesthesia)

namespace Audio
{
	void FAudioSynesthesiaModule::StartupModule()
	{
		AUDIO_ANALYSIS_LLM_SCOPE
		FModuleManager::Get().LoadModuleChecked<IModuleInterface>("AudioSynesthesiaCore");
		FModuleManager::Get().LoadModuleChecked<FAudioAnalyzerModule>("AudioAnalyzer");
	}

	void FAudioSynesthesiaModule::ShutdownModule()
	{
	}
}
