// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSynesthesiaNRT.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioSynesthesiaNRT)


const TArray<FText>& UAudioSynesthesiaNRTSettings::GetAssetActionSubmenus() const
{
	static const TArray<FText> SubMenus
	{
		NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AssetSoundAnalysisSubmenu", "Analysis")
	};

	return SubMenus;
}

#if WITH_EDITOR
FColor UAudioSynesthesiaNRTSettings::GetTypeColor() const
{
	return FColor(200.0f, 150.0f, 200.0f);
}
#endif

const TArray<FText>& UAudioSynesthesiaNRT::GetAssetActionSubmenus() const
{
	static const TArray<FText> SubMenus
	{
		NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AssetSoundAnalysisSubmenu", "Analysis")
	};

	return SubMenus;
}

#if WITH_EDITOR
FColor UAudioSynesthesiaNRT::GetTypeColor() const
{
	return FColor(200.0f, 150.0f, 200.0f);
}
#endif

