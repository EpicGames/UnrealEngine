// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstantQ.h"
#include "ConstantQFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConstantQ)


namespace ConstantQPrivate
{
	/** Convenience template function for converting from blueprint types to audio types */
	template<typename InType, typename OutType>
	OutType ConvertType(const TMap<InType, OutType>& InMap, const InType& InValue, const OutType& InDefaultValue)
	{
		checkf(InMap.Contains(InValue), TEXT("Unhandled value conversion"));

		if (InMap.Contains(InValue))
		{
			return InMap[InValue];
		}

		return InDefaultValue;
	}


	// EConstantQFFTSizeEnum to int
	int32 BPConstantQFFTSizeEnumToInt(const EConstantQFFTSizeEnum InFFTSizeEnum)
	{
		static const TMap<EConstantQFFTSizeEnum, int32> FFTSizeMap =
		{
			{EConstantQFFTSizeEnum::Min, 		64},
			{EConstantQFFTSizeEnum::XXSmall, 	128},
			{EConstantQFFTSizeEnum::XSmall, 	256},
			{EConstantQFFTSizeEnum::Small,		512},
			{EConstantQFFTSizeEnum::Medium, 	1024},
			{EConstantQFFTSizeEnum::Large,		2048},
			{EConstantQFFTSizeEnum::XLarge, 	4096},
			{EConstantQFFTSizeEnum::XXLarge, 	8192},
			{EConstantQFFTSizeEnum::Max, 		16384}
		};

		return ConvertType(FFTSizeMap, InFFTSizeEnum, 2048);

	}

	// EFFTWindwoType to Audio::EWindowType
	Audio::EWindowType BPWindowTypeToAudioWindowType(const EFFTWindowType InWindowType)
	{
		static const TMap<EFFTWindowType, Audio::EWindowType> WindowTypeMap =
		{
			{EFFTWindowType::None, 		Audio::EWindowType::None},
			{EFFTWindowType::Hamming, 	Audio::EWindowType::Hamming},
			{EFFTWindowType::Hann, 		Audio::EWindowType::Hann},
			{EFFTWindowType::Blackman,	Audio::EWindowType::Blackman}
		};

		return ConvertType(WindowTypeMap, InWindowType, Audio::EWindowType::None);
	}

	// ESpectrumType to Audio::ESpectrumType
	Audio::ESpectrumType BPSpectrumTypeToAudioSpectrumType(const EAudioSpectrumType InSpectrumType)
	{
		static const TMap<EAudioSpectrumType, Audio::ESpectrumType> SpectrumTypeMap =
		{
			{EAudioSpectrumType::MagnitudeSpectrum,	Audio::ESpectrumType::MagnitudeSpectrum},
			{EAudioSpectrumType::PowerSpectrum, 	Audio::ESpectrumType::PowerSpectrum}
		};

		return ConvertType(SpectrumTypeMap, InSpectrumType, Audio::ESpectrumType::PowerSpectrum);
	}

	// EContantQNormalizationEnum to Audio::EPseudoConstantQNormalization
	Audio::EPseudoConstantQNormalization BPCQTNormalizationToAudioCQTNormalization(const EConstantQNormalizationEnum InNormalization)
	{
		static const TMap<EConstantQNormalizationEnum, Audio::EPseudoConstantQNormalization> NormalizationMap =
		{
			{EConstantQNormalizationEnum::EqualAmplitude, 		Audio::EPseudoConstantQNormalization::EqualAmplitude},
			{EConstantQNormalizationEnum::EqualEuclideanNorm,	Audio::EPseudoConstantQNormalization::EqualEuclideanNorm},
			{EConstantQNormalizationEnum::EqualEnergy, 			Audio::EPseudoConstantQNormalization::EqualEnergy}
		};

		return ConvertType(NormalizationMap, InNormalization, Audio::EPseudoConstantQNormalization::EqualEnergy);
	}
}


/** Convert UConstantQSettings to FConstantQSettings */
TUniquePtr<Audio::IAnalyzerSettings> UConstantQSettings::GetSettings(const float InSampleRate, const int32 InNumChannels) const
{
	using namespace ConstantQPrivate;

	TUniquePtr<Audio::FConstantQSettings> Settings = MakeUnique<Audio::FConstantQSettings>();

	Settings->AnalysisPeriodInSeconds = AnalysisPeriodInSeconds;
	Settings->bDownmixToMono = bDownmixToMono;
	Settings->FFTSize = BPConstantQFFTSizeEnumToInt(FFTSize);
	Settings->WindowType = BPWindowTypeToAudioWindowType(WindowType);
	Settings->SpectrumType = BPSpectrumTypeToAudioSpectrumType(SpectrumType);
	Settings->NumBands = NumBands;
	Settings->NumBandsPerOctave = NumBandsPerOctave;
	Settings->KernelLowestCenterFreq = StartingFrequencyHz;
	Settings->BandWidthStretch = BandWidthStretch;
	Settings->Normalization = BPCQTNormalizationToAudioCQTNormalization(CQTNormalization);
	Settings->Scaling = (SpectrumType == EAudioSpectrumType::Decibel) ? Audio::EConstantQScaling::Decibel : Audio::EConstantQScaling::Linear;

	return Settings;
}

#if WITH_EDITOR
FText UConstantQSettings::GetAssetActionName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AssetSoundSynesthesiaConstantQSettings", "Synesthesia Real-Time Settings (ConstantQ)");
}

UClass* UConstantQSettings::GetSupportedClass() const
{
	return UConstantQSettings::StaticClass();
}
#endif // WITH_EDITOR

UConstantQAnalyzer::UConstantQAnalyzer()
{
	Settings = CreateDefaultSubobject<UConstantQSettings>(TEXT("DefaultConstantQSettings"));
}

TUniquePtr<Audio::IAnalyzerSettings> UConstantQAnalyzer::GetSettings(const int32 InSampleRate, const int32 InNumChannels) const
{
	TUniquePtr<Audio::IAnalyzerSettings> AnalyzerSettings;

	if (Settings)
	{
		AnalyzerSettings = Settings->GetSettings(InSampleRate, InNumChannels);
	}

	return AnalyzerSettings;
}

static TArray<FConstantQResults> ConvertToBlueprintResults(UConstantQSettings* Settings, const TArray<Audio::FConstantQFrame>& InConstantQFrameArray)
{
	TArray<FConstantQResults> ResultsArray;

	for (const Audio::FConstantQFrame& ConstantQFrame : InConstantQFrameArray)
	{
		FConstantQResults NewResults;

		NewResults.SpectrumValues = ConstantQFrame.Spectrum;
		NewResults.TimeSeconds = ConstantQFrame.Timestamp;

		ResultsArray.Add(NewResults);
	}

	// Sort by time priority (lowest priority/latest first).
	ResultsArray.Sort([](const FConstantQResults& A, const FConstantQResults& B)
		{
			return A.TimeSeconds < B.TimeSeconds;
		});

	return MoveTemp(ResultsArray);
}

void UConstantQAnalyzer::BroadcastResults()
{
	TUniquePtr<const Audio::FConstantQResult> ConstantQResults = GetResults<Audio::FConstantQResult>();
	if (!ConstantQResults.IsValid())
	{
		return;
	}

	const int32 NumChannels = ConstantQResults->GetNumChannels();

	if (NumChannels <= 0)
	{
		return;
	}

	bool bIsOnConstantQResultsBound = OnConstantQResults.IsBound() || OnConstantQResultsNative.IsBound();
	bool bIsOnLatestConstantQResultsBound = OnLatestConstantQResults.IsBound() || OnLatestConstantQResultsNative.IsBound();

	if (bIsOnConstantQResultsBound || bIsOnLatestConstantQResultsBound)
	{
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
		{
			const TArray<Audio::FConstantQFrame>& ConstantQFrameArray = ConstantQResults->GetFramesForChannel(ChannelIndex);
			if (ConstantQFrameArray.Num() > 0)
			{
				// Sorts results by timestamp
				TArray<FConstantQResults> Results = ConvertToBlueprintResults(Settings, ConstantQFrameArray);
				ensure(Results.Num() > 0);

				OnConstantQResults.Broadcast(ChannelIndex, Results);
				OnConstantQResultsNative.Broadcast(this, ChannelIndex, Results);

				OnLatestConstantQResults.Broadcast(ChannelIndex, Results[Results.Num() - 1]);
				OnLatestConstantQResultsNative.Broadcast(this, ChannelIndex, Results[Results.Num() - 1]);
			}
		}
	}
}

void UConstantQAnalyzer::GetCenterFrequencies(TArray<float>& OutCenterFrequencies)
{
	const int32 NumFrequencies = GetNumCenterFrequencies();
	if (NumFrequencies == OutCenterFrequencies.Num())
	{
		const float PitchStep = FMath::Pow(2.0f, 1.0f / Settings->NumBandsPerOctave);

		float CurrentFrequency = Settings->StartingFrequencyHz;
		for (int32 i = 0; i < NumFrequencies; ++i)
		{
			OutCenterFrequencies[i] = CurrentFrequency;
			CurrentFrequency *= PitchStep;
		}
	}
}

const int32 UConstantQAnalyzer::GetNumCenterFrequencies() const
{
	return Settings->NumBands;
}

FName UConstantQAnalyzer::GetAnalyzerFactoryName() const
{
	static const FName FactoryName(TEXT("ConstantQFactory"));
	return FactoryName;
}
