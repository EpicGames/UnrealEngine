// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstantQNRT.h"
#include "ConstantQNRTFactory.h"
#include "AudioSynesthesiaLog.h"
#include "DSP/FloatArrayMath.h"
#include "InterpolateSorted.h"
#include "Sound/SoundSubmix.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConstantQNRT)


namespace
{
	/** Convenience template function for converting from blueprint types to audio types */
	template<typename InType, typename OutType>
	OutType ConvertType(const TMap<InType, OutType>& InMap, const InType& InValue, const OutType& InDefaultValue)
	{
		checkf(InMap.Contains(InValue), TEXT("Unhandled value conversion"));

		if (InMap.Contains(InValue))
		{
			return InMap[InValue];
		}

		return InDefaultValue;
	}
	

	// EConstantQFFTSizeEnum to int
	int32 BPConstantQFFTSizeEnumToInt(EConstantQFFTSizeEnum InFFTSizeEnum)
	{
		static const TMap<EConstantQFFTSizeEnum, int32> FFTSizeMap = {
			{EConstantQFFTSizeEnum::Min, 		64},
			{EConstantQFFTSizeEnum::XXSmall, 	128},
			{EConstantQFFTSizeEnum::XSmall, 	256},
			{EConstantQFFTSizeEnum::Small,		512},
			{EConstantQFFTSizeEnum::Medium, 	1024},
			{EConstantQFFTSizeEnum::Large,		2048},
			{EConstantQFFTSizeEnum::XLarge, 	4096},
			{EConstantQFFTSizeEnum::XXLarge, 	8192},
			{EConstantQFFTSizeEnum::Max, 		16384}
		};

		return ConvertType(FFTSizeMap, InFFTSizeEnum, 2048);

	}

	// EFFTWindwoType to Audio::EWindowType
	Audio::EWindowType BPWindowTypeToAudioWindowType(EFFTWindowType InWindowType)
	{
		static const TMap<EFFTWindowType, Audio::EWindowType> WindowTypeMap = {
			{EFFTWindowType::None, 		Audio::EWindowType::None},
			{EFFTWindowType::Hamming, 	Audio::EWindowType::Hamming},
			{EFFTWindowType::Hann, 		Audio::EWindowType::Hann},
			{EFFTWindowType::Blackman,	Audio::EWindowType::Blackman}
		};

		return ConvertType(WindowTypeMap, InWindowType, Audio::EWindowType::None);
	}

	// ESpectrumType to Audio::ESpectrumType
	Audio::ESpectrumType BPSpectrumTypeToAudioSpectrumType(EAudioSpectrumType InSpectrumType)
	{
		static const TMap<EAudioSpectrumType, Audio::ESpectrumType> SpectrumTypeMap = {
			{EAudioSpectrumType::MagnitudeSpectrum,	Audio::ESpectrumType::MagnitudeSpectrum},
			{EAudioSpectrumType::PowerSpectrum, 	Audio::ESpectrumType::PowerSpectrum}
		};

		return ConvertType(SpectrumTypeMap, InSpectrumType, Audio::ESpectrumType::PowerSpectrum);
	}

	// EContantQNormalizationEnum to Audio::EPseudoConstantQNormalization
	Audio::EPseudoConstantQNormalization BPCQTNormalizationToAudioCQTNormalization(EConstantQNormalizationEnum InNormalization)
	{
		static const TMap<EConstantQNormalizationEnum, Audio::EPseudoConstantQNormalization> NormalizationMap = {
			{EConstantQNormalizationEnum::EqualAmplitude, 		Audio::EPseudoConstantQNormalization::EqualAmplitude},
			{EConstantQNormalizationEnum::EqualEuclideanNorm,	Audio::EPseudoConstantQNormalization::EqualEuclideanNorm},
			{EConstantQNormalizationEnum::EqualEnergy, 			Audio::EPseudoConstantQNormalization::EqualEnergy}
		};

		return ConvertType(NormalizationMap, InNormalization, Audio::EPseudoConstantQNormalization::EqualEnergy);
	}

	// Interpolates each value in input frame at timestamp
	void InterpolateSpectrumAtTimestamp(TArrayView<const Audio::FConstantQFrame> InFrames, float InTimestamp, TArray<float>& OutSpectrum)
	{
		OutSpectrum.Reset();

		int32 LowerIndex = INDEX_NONE;
		int32 UpperIndex = INDEX_NONE;
		float Alpha = 0.f;

		// get interpolation values based on timestmaps
		GetInterpolationParametersAtTimestamp(InFrames, InTimestamp, LowerIndex, UpperIndex, Alpha);

		if ((INDEX_NONE == LowerIndex) || (INDEX_NONE == UpperIndex))
		{
			// there was an error getting the interpolation parameters
			return;
		}
		
		const Audio::FConstantQFrame& LowerFrame = InFrames[LowerIndex];
		const Audio::FConstantQFrame& UpperFrame = InFrames[UpperIndex];

		const int32 Num = LowerFrame.Spectrum.Num();

		OutSpectrum.AddUninitialized(Num);

		// Linearly interpolate between upper and lower frame spectrums.
		const float* LowerSpectrumData = LowerFrame.Spectrum.GetData();
		const float* UpperSpectrumData = UpperFrame.Spectrum.GetData();
		float* OutSpectrumData = OutSpectrum.GetData();

		for (int32 i = 0; i < Num; i++)
		{
			OutSpectrumData[i] = FMath::Lerp(LowerSpectrumData[i], UpperSpectrumData[i], Alpha);
		}
	}
}

/***************************************************************************/
/*********************    UConstantQNRTSettings     ************************/
/***************************************************************************/

UConstantQNRTSettings::UConstantQNRTSettings()
: 	StartingFrequency(40.f)
,	NumBands(48)
,	NumBandsPerOctave(12.f)
,	AnalysisPeriod(0.01f)
,	bDownmixToMono(false)
,	FFTSize(EConstantQFFTSizeEnum::XLarge)
,	WindowType(EFFTWindowType::Blackman)
,	SpectrumType(EAudioSpectrumType::PowerSpectrum)
,	BandWidthStretch(1.f)
,	CQTNormalization(EConstantQNormalizationEnum::EqualEnergy)
,	NoiseFloorDb(-60.f)
{}

/** Convert UConstantQNRTSettings to FConstantQNRTSettings */
TUniquePtr<Audio::IAnalyzerNRTSettings> UConstantQNRTSettings::GetSettings(const float InSampleRate, const int32 InNumChannels) const
{
	TUniquePtr<Audio::FConstantQNRTSettings> Settings = MakeUnique<Audio::FConstantQNRTSettings>();

	Settings->AnalysisPeriod 			= AnalysisPeriod;
	Settings->bDownmixToMono 			= bDownmixToMono;
	Settings->FFTSize 					= BPConstantQFFTSizeEnumToInt(FFTSize);
	Settings->WindowType 				= BPWindowTypeToAudioWindowType(WindowType);
	Settings->SpectrumType 				= BPSpectrumTypeToAudioSpectrumType(SpectrumType);
	Settings->NumBands 					= NumBands;
	Settings->NumBandsPerOctave 		= NumBandsPerOctave;
	Settings->KernelLowestCenterFreq	= StartingFrequency;
	Settings->BandWidthStretch 			= BandWidthStretch;
	Settings->Normalization 			= BPCQTNormalizationToAudioCQTNormalization(CQTNormalization);
	Settings->Scaling					= Audio::EConstantQScaling::Decibel;

	return Settings;
}

#if WITH_EDITOR
FText UConstantQNRTSettings::GetAssetActionName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AssetSoundSynesthesiaConstantQNRTSettings", "Synesthesia NRT Settings (ConstantQ)");
}

UClass* UConstantQNRTSettings::GetSupportedClass() const
{
	return UConstantQNRTSettings::StaticClass();
}
#endif

/***************************************************************************/
/*********************        UConstantQNRT         ************************/
/***************************************************************************/

UConstantQNRT::UConstantQNRT()
{
	Settings = CreateDefaultSubobject<UConstantQNRTSettings>(TEXT("DefaultConstantQNRTSettings"));

#if WITH_EDITOR
	// Bind settings to audio analyze so changes to default settings will trigger analysis.
	SetSettingsDelegate(Settings);
#endif
}


void UConstantQNRT::GetChannelConstantQAtTime(const float InSeconds, const int32 InChannel, TArray<float>& OutConstantQ) const
{
	OutConstantQ.Reset();

	TSharedPtr<const Audio::FConstantQNRTResult, ESPMode::ThreadSafe> ConstantQResult = GetResult<Audio::FConstantQNRTResult>();

	if (ConstantQResult.IsValid())
	{
		// ConstantQResults should always be sorted if in UConstantQNRT
		check(ConstantQResult->IsSortedChronologically());

		if (!ConstantQResult->ContainsChannel(InChannel))
		{
			UE_LOGF(LogAudioSynesthesia, Warning, "ConstantQNRT does not contain channel %d", InChannel);
			return;
		}

		const TArray<Audio::FConstantQFrame>& ConstantQArray = ConstantQResult->GetFramesForChannel(InChannel);

		InterpolateSpectrumAtTimestamp(ConstantQArray, InSeconds, OutConstantQ);
	}
}

void UConstantQNRT::GetNormalizedChannelConstantQAtTime(const float InSeconds, const int32 InChannelIdx, TArray<float>& OutConstantQ) const
{
	OutConstantQ.Reset();

	TSharedPtr<const Audio::FConstantQNRTResult, ESPMode::ThreadSafe> ConstantQResult = GetResult<Audio::FConstantQNRTResult>();

	if (ConstantQResult.IsValid())
	{
		// ConstantQResults should always be sorted if in UConstantQNRT
		check(ConstantQResult->IsSortedChronologically());

		if (!ConstantQResult->ContainsChannel(InChannelIdx))
		{
			UE_LOGF(LogAudioSynesthesia, Warning, "ConstantQNRT does not contain channel %d", InChannelIdx);
			return;
		}

		const TArray<Audio::FConstantQFrame>& ConstantQArray = ConstantQResult->GetFramesForChannel(InChannelIdx);

		InterpolateSpectrumAtTimestamp(ConstantQArray, InSeconds, OutConstantQ);

		// Normalize output by subtracting minimum and dividing by range
		Audio::ArraySubtractByConstantInPlace(OutConstantQ, Settings->NoiseFloorDb);

		FFloatInterval ConstantQInterval = ConstantQResult->GetChannelConstantQInterval(InChannelIdx);

		const float ConstantQRange = ConstantQInterval.Max - Settings->NoiseFloorDb;

		if (ConstantQRange > SMALL_NUMBER)
		{
			const float Scaling = 1.f / ConstantQRange;
			Audio::ArrayMultiplyByConstantInPlace(OutConstantQ, Scaling);
		}
		else 
		{
			// The range is too small or negative. Set output values to zero.
			if (OutConstantQ.Num() > 0)
			{
				FMemory::Memset(OutConstantQ.GetData(), 0, sizeof(float) * OutConstantQ.Num());
			}
		}

		Audio::ArrayClampInPlace(OutConstantQ, 0.f, 1.f);
	}
}

TUniquePtr<Audio::IAnalyzerNRTSettings> UConstantQNRT::GetSettings(const float InSampleRate, const int32 InNumChannels) const
{
	TUniquePtr<Audio::IAnalyzerNRTSettings> AnalyzerSettings;

	if (Settings)
	{
		AnalyzerSettings = Settings->GetSettings(InSampleRate, InNumChannels);	
	}

	return AnalyzerSettings;
}

#if WITH_EDITOR
FText UConstantQNRT::GetAssetActionName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AssetSoundSynesthesiaConstantQNRT", "Synesthesia NRT (ConstantQ)");
}

UClass* UConstantQNRT::GetSupportedClass() const
{
	return UConstantQNRT::StaticClass();
}

bool UConstantQNRT::ShouldEventTriggerAnalysis(FPropertyChangedEvent& PropertyChangeEvent)
{
	return Settings != nullptr && Super::ShouldEventTriggerAnalysis(PropertyChangeEvent);
}
#endif // WITH_EDITOR

FName UConstantQNRT::GetAnalyzerNRTFactoryName() const 
{
	static const FName FactoryName(TEXT("ConstantQNRTFactory"));
	return FactoryName;
}


