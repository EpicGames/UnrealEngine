// Copyright Epic Games, Inc. All Rights Reserved.

#include "LKFS.h"
#include "LKFSFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LKFS)

TUniquePtr<Audio::IAnalyzerSettings> ULKFSSettings::GetSettings(const int32 InSampleRate, const int32 InNumChannels) const
{
	TUniquePtr<Audio::FLKFSSettings> Settings = MakeUnique<Audio::FLKFSSettings>();

	Settings->AnalysisPeriod = AnalysisPeriod;
	Settings->AnalysisWindowDuration = AnalysisWindowDuration;
	Settings->ShortTermLoudnessDuration = ShortTermLoudnessDuration;
	Settings->IntegratedLoudnessAnalysisPeriod = IntegratedLoudnessAnalysisPeriod;
	Settings->IntegratedLoudnessDuration = IntegratedLoudnessDuration;
	Settings->bCalculatePerChannelLoudnessRange = bCalculatePerChannelLoudnessRange;
	Settings->bCalculateOverallLoudnessRange = bCalculateOverallLoudnessRange;

	return Settings;
}

#if WITH_EDITOR
FText ULKFSSettings::GetAssetActionName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AssetSoundSynesthesiaLKFSSettings", "Synesthesia Real-Time Settings (LKFS)");
}

UClass* ULKFSSettings::GetSupportedClass() const
{
	return ULKFSSettings::StaticClass();
}
#endif

ULKFSAnalyzer::ULKFSAnalyzer()
{
	Settings = CreateDefaultSubobject<ULKFSSettings>(TEXT("DefaultLKFSSettings"));
}

TUniquePtr<Audio::IAnalyzerSettings> ULKFSAnalyzer::GetSettings(const int32 InSampleRate, const int32 InNumChannels) const
{
	TUniquePtr<Audio::IAnalyzerSettings> AnalyzerSettings;

	if (Settings)
	{
		AnalyzerSettings = Settings->GetSettings(InSampleRate, InNumChannels);
	}

	return AnalyzerSettings;
}

void ULKFSAnalyzer::BroadcastResults()
{
	TUniquePtr<const Audio::FLKFSAnalyzerResult> AnalyzerResults = GetResults<Audio::FLKFSAnalyzerResult>();
	if (!AnalyzerResults.IsValid())
	{
		return;
	}

	int32 NumChannels = AnalyzerResults->GetNumChannels();

	if (NumChannels > 0)
	{
		// Broadcast overall results
		bool bIsOnOverallLKFSResultsBound = OnOverallLKFSResults.IsBound() || OnOverallLKFSResultsNative.IsBound();
		bool bIsOnLatestOverallLKFSResultsBound = OnLatestOverallLKFSResults.IsBound() || OnLatestOverallLKFSResultsNative.IsBound();
		if (bIsOnOverallLKFSResultsBound || bIsOnLatestOverallLKFSResultsBound)
		{
			const TArray<FLKFSResults>& OverallLKFSArray = AnalyzerResults->GetLoudnessResults();
			if (OverallLKFSArray.Num() > 0)
			{
				OnOverallLKFSResults.Broadcast(OverallLKFSArray);
				OnOverallLKFSResultsNative.Broadcast(this, OverallLKFSArray);

				const FLKFSResults& Latest = OverallLKFSArray[OverallLKFSArray.Num() - 1];
				OnLatestOverallLKFSResults.Broadcast(Latest);
				OnLatestOverallLKFSResultsNative.Broadcast(this, Latest);
			}
		}

		// Broadcast per channel results
		bool bIsOnPerChannelLKFSResultsBound = OnPerChannelLKFSResults.IsBound() || OnPerChannelLKFSResultsNative.IsBound();
		bool bIsOnLatestPerChannelLKFSResultsBound = OnLatestPerChannelLKFSResults.IsBound() || OnLatestPerChannelLKFSResultsNative.IsBound();

		if (bIsOnPerChannelLKFSResultsBound || bIsOnLatestPerChannelLKFSResultsBound)
		{
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
				const TArray<FLKFSResults>& LKFSArray = AnalyzerResults->GetChannelLoudnessResults(ChannelIndex);
				if (LKFSArray.Num() > 0)
				{
					OnPerChannelLKFSResults.Broadcast(ChannelIndex, LKFSArray);
					OnPerChannelLKFSResultsNative.Broadcast(this, ChannelIndex, LKFSArray);

					const FLKFSResults& Latest = LKFSArray[LKFSArray.Num() - 1];
					OnLatestPerChannelLKFSResults.Broadcast(ChannelIndex, Latest);
					OnLatestPerChannelLKFSResultsNative.Broadcast(this, ChannelIndex, Latest);
				}
			}
		}
	}
}

FName ULKFSAnalyzer::GetAnalyzerFactoryName() const
{
	static const FName FactoryName(TEXT("LKFSFactory"));
	return FactoryName;
}
