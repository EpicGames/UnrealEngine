// Copyright Epic Games, Inc. All Rights Reserved.

#include "LKFSNRT.h"

#include "FindNearestByTimestamp.h"
#include "LKFSNRTFactory.h"
#include "InterpolateSorted.h"
#include "AudioSynesthesiaLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LKFSNRT)

namespace LKFSNRTPrivate
{
	float InterpolateLoudness(TArrayView<const FLKFSNRTResults> InLoudnessArray, const float InTimestamp)
	{
		int32 LowerIndex = INDEX_NONE;
		int32 UpperIndex = INDEX_NONE;
		float Alpha = 0.f;

		Audio::GetInterpolationParametersAtTimestamp(InLoudnessArray, InTimestamp, LowerIndex, UpperIndex, Alpha);

		if ((INDEX_NONE != LowerIndex) && (INDEX_NONE != UpperIndex))
		{
			return FMath::Lerp(InLoudnessArray[LowerIndex].Loudness, InLoudnessArray[UpperIndex].Loudness, Alpha);
		}

		return 0.f;
	}
}


/***************************************************************************/
/**********************    ULKFSNRTSettings     ************************/
/***************************************************************************/

ULKFSNRTSettings::ULKFSNRTSettings()
{}

TUniquePtr<Audio::IAnalyzerNRTSettings> ULKFSNRTSettings::GetSettings(const float InSampleRate, const int32 InNumChannels) const
{
	using namespace Audio;

	TUniquePtr<FLKFSNRTSettings> OutSettings = MakeUnique<FLKFSNRTSettings>();

	// lkfs is generally done in time domain, but the loudness analyzer works in 
	// the spectral domain. Choosing this FFT small enough to allow for proper 
	// spectral-temporal granularity. If the FFT size is too large it can become 
	// larger than the AnalysisWindowDuration. If the FFT is too small, the perceptual
	// weighting in the frequency domain looses accuracy. 
	OutSettings->LoudnessAnalyzerSettings.FFTSize = 2048;
	OutSettings->LoudnessAnalyzerSettings.WindowType = Audio::EWindowType::Blackman;
	OutSettings->LoudnessAnalyzerSettings.MinAnalysisFrequency = 20.f;
	OutSettings->LoudnessAnalyzerSettings.MaxAnalysisFrequency = 20000.f;
	OutSettings->LoudnessAnalyzerSettings.WindowSize = -1;
	// Old loudness analyzer settings had incorrect method for doing loudness metrics. To match LKFS standard we have to use a corrected scaling method.
	OutSettings->LoudnessAnalyzerSettings.ScalingMethod = ELoudnessAnalyzerScalingMethod::Corrected;
	OutSettings->LoudnessAnalyzerSettings.LoudnessCurveType = ELoudnessCurveType::K,

	OutSettings->AnalysisPeriod = AnalysisPeriod;
	OutSettings->AnalysisWindowDuration = AnalysisWindowDuration;
	OutSettings->ShortTermLoudnessDuration = ShortTermLoudnessDuration;
		
	return OutSettings;
}

#if WITH_EDITOR
FText ULKFSNRTSettings::GetAssetActionName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AssetSoundSynesthesiaLKFSNRTSettings", "Synesthesia NRT Settings (LKFS / LUFS)");
}

UClass* ULKFSNRTSettings::GetSupportedClass() const
{
	return ULKFSNRTSettings::StaticClass();
}
#endif

/***************************************************************************/
/**********************        ULKFSNRT         ************************/
/***************************************************************************/

ULKFSNRT::ULKFSNRT()
{
	Settings = CreateDefaultSubobject<ULKFSNRTSettings>(TEXT("DefaultLKFSNRTSettings"));
#if WITH_EDITOR
	// Bind settings to audio analyze so changes to default settings will trigger analysis.
	SetSettingsDelegate(Settings);
#endif
}

void ULKFSNRT::GetLoudnessAtTime(const float InSeconds, float& OutLoudness) const
{
	GetChannelLoudnessAtTime(InSeconds, Audio::FLKFSNRTResult::ChannelIndexOverall, OutLoudness);
}

void ULKFSNRT::GetChannelLoudnessAtTime(const float InSeconds, const int32 InChannel, float& OutLoudness) const
{
	OutLoudness = 0.0f;

	TSharedPtr<const Audio::FLKFSNRTResult, ESPMode::ThreadSafe> LoudnessResult = GetResult<Audio::FLKFSNRTResult>();

	if (LoudnessResult.IsValid())
	{
		// The loudness result should never used here if it is not already sorted.
		check(LoudnessResult->IsSortedChronologically());

		if (!LoudnessResult->ContainsChannel(InChannel))
		{
			UE_LOGF(LogAudioSynesthesia, Warning, "LKFSNRT does not contain channel %d", InChannel);
			return;
		}

		TArrayView<const FLKFSNRTResults> LoudnessArray = LoudnessResult->GetChannelLoudnessArray(InChannel);

		OutLoudness = LKFSNRTPrivate::InterpolateLoudness(LoudnessArray, InSeconds);
	}
}

TArray<FLKFSNRTResults> ULKFSNRT::GetLoudnessDataForChannel(int32 InChannel) const
{
	TSharedPtr<const Audio::FLKFSNRTResult, ESPMode::ThreadSafe> LoudnessResult = GetResult<Audio::FLKFSNRTResult>();
	if (LoudnessResult.IsValid())
	{
		// The loudness result should never used here if it is not already sorted.
		check(LoudnessResult->IsSortedChronologically());

		if (!LoudnessResult->ContainsChannel(InChannel))
		{
			UE_LOGF(LogAudioSynesthesia, Warning, "LKFSNRT does not contain channel %d", InChannel);
			return {};
		}

		return TArray<FLKFSNRTResults>(LoudnessResult->GetChannelLoudnessArray(InChannel));
	}
	return {};
}

TArray<FLKFSNRTResults> ULKFSNRT::GetLoudnessData() const
{
	return GetLoudnessDataForChannel(Audio::FLKFSNRTResult::ChannelIndexOverall);
}

FLKFSNRTResults ULKFSNRT::GetLoudnessDataForChannelAtTime(const float InSeconds, int32 InChannel) const
{
	TSharedPtr<const Audio::FLKFSNRTResult, ESPMode::ThreadSafe> LoudnessResult = GetResult<Audio::FLKFSNRTResult>();
	if (LoudnessResult.IsValid())
	{
		// The loudness result should never used here if it is not already sorted.
		check(LoudnessResult->IsSortedChronologically());

		if (!LoudnessResult->ContainsChannel(InChannel))
		{
			UE_LOGF(LogAudioSynesthesia, Warning, "LKFSNRT does not contain channel %d", InChannel);
			return {};
		}

		TArrayView<const FLKFSNRTResults> LoudnessArray = LoudnessResult->GetChannelLoudnessArray(InChannel);
		if (const FLKFSNRTResults* Datum = Audio::FindNearestByTimestamp<const FLKFSNRTResults>(LoudnessArray, InSeconds))
		{
			return *Datum;
		}
	}
	return {};
}

FLKFSNRTResults ULKFSNRT::GetLoudnessDataAtTime(const float InSeconds) const
{
	return GetLoudnessDataForChannelAtTime(InSeconds, Audio::FLKFSNRTResult::ChannelIndexOverall);
}

float ULKFSNRT::GetIntegratedLoudnessForChannel(int32 InChannel) const
{
	TSharedPtr<const Audio::FLKFSNRTResult, ESPMode::ThreadSafe> LoudnessResult = GetResult<Audio::FLKFSNRTResult>();
	if (LoudnessResult.IsValid())
	{
		if (const FLKFSNRTAggregateStats* Stats = LoudnessResult->FindAggregateLoudnessStats(InChannel))
		{
			return Stats->IntegratedLoudness;
		}
		else
		{
			UE_LOGF(LogAudioSynesthesia, Warning, "LKFSNRT does not contain integrated loudness for channel %d", InChannel);
		}
	}
	return FLKFSNRTAggregateStats::InvalidLoudness;
}

float ULKFSNRT::GetIntegratedLoudness() const
{
	return GetIntegratedLoudnessForChannel(Audio::FLKFSNRTResult::ChannelIndexOverall);
}

float ULKFSNRT::GetGatedLoudnessForChannel(int32 InChannel) const
{
	TSharedPtr<const Audio::FLKFSNRTResult, ESPMode::ThreadSafe> LoudnessResult = GetResult<Audio::FLKFSNRTResult>();
	if (LoudnessResult.IsValid())
	{
		if (const FLKFSNRTAggregateStats* Stats = LoudnessResult->FindAggregateLoudnessStats(InChannel))
		{
			return Stats->GatedLoudness;
		}
		else
		{
			UE_LOGF(LogAudioSynesthesia, Warning, "LKFSNRT does not contain gated loudness for channel %d", InChannel);
		}
	}
	return FLKFSNRTAggregateStats::InvalidLoudness;
}

float ULKFSNRT::GetGatedLoudness() const
{
	return GetGatedLoudnessForChannel(Audio::FLKFSNRTResult::ChannelIndexOverall);
}

TUniquePtr<Audio::IAnalyzerNRTSettings> ULKFSNRT::GetSettings(const float InSampleRate, const int32 InNumChannels) const
{
	TUniquePtr<Audio::IAnalyzerNRTSettings> AnalyzerSettings;

	if (Settings)
	{
		AnalyzerSettings = Settings->GetSettings(InSampleRate, InNumChannels);	
	}

	return AnalyzerSettings;
}

#if WITH_EDITOR
FText ULKFSNRT::GetAssetActionName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AssetSoundSynesthesiaLKFSNRT", "Synesthesia NRT (LKFS / LUFS)");
}

UClass* ULKFSNRT::GetSupportedClass() const
{
	return ULKFSNRT::StaticClass();
}

bool ULKFSNRT::ShouldEventTriggerAnalysis(FPropertyChangedEvent& PropertyChangeEvent)
{
	return Settings != nullptr && Super::ShouldEventTriggerAnalysis(PropertyChangeEvent);
}
#endif // WITH_EDITOR

FName ULKFSNRT::GetAnalyzerNRTFactoryName() const
{
	static const FName FactoryName(TEXT("LKFSNRTFactory"));
	return FactoryName;
}


