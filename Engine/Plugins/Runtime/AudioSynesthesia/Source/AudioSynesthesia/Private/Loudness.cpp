// Copyright Epic Games, Inc. All Rights Reserved.

#include "Loudness.h"
#include "LoudnessFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Loudness)

TUniquePtr<Audio::IAnalyzerSettings> ULoudnessSettings::GetSettings(const int32 InSampleRate, const int32 InNumChannels) const
{
	TUniquePtr<Audio::FLoudnessSettings> Settings = MakeUnique<Audio::FLoudnessSettings>();

	Settings->AnalysisPeriod = AnalysisPeriod;
	Settings->MinAnalysisFrequency = MinimumFrequency;
	Settings->MaxAnalysisFrequency = MaximumFrequency;

	switch (CurveType)
	{
	case ELoudnessCurveTypeEnum::A:
		Settings->LoudnessCurveType = Audio::ELoudnessCurveType::A;
		break;

	case ELoudnessCurveTypeEnum::B:
		Settings->LoudnessCurveType = Audio::ELoudnessCurveType::B;
		break;

	case ELoudnessCurveTypeEnum::C:
		Settings->LoudnessCurveType = Audio::ELoudnessCurveType::C;
		break;

	case ELoudnessCurveTypeEnum::D:
		Settings->LoudnessCurveType = Audio::ELoudnessCurveType::D;
		break;

	case ELoudnessCurveTypeEnum::K:
		Settings->LoudnessCurveType = Audio::ELoudnessCurveType::K;
		break;

	case ELoudnessCurveTypeEnum::None:
	default:
		Settings->LoudnessCurveType = Audio::ELoudnessCurveType::None;
		break;
	}

	return Settings;
}

#if WITH_EDITOR
FText ULoudnessSettings::GetAssetActionName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AssetSoundSynesthesiaLoudnessSettings", "Synesthesia Real-Time Settings (Loudness)");
}

UClass* ULoudnessSettings::GetSupportedClass() const
{
	return ULoudnessSettings::StaticClass();
}
#endif

ULoudnessAnalyzer::ULoudnessAnalyzer()
{
	if (!Settings)
	{
		Settings = CreateDefaultSubobject<ULoudnessSettings>(TEXT("DefaultLoudnessSettings"));
	}
}

TUniquePtr<Audio::IAnalyzerSettings> ULoudnessAnalyzer::GetSettings(const int32 InSampleRate, const int32 InNumChannels) const
{
	TUniquePtr<Audio::IAnalyzerSettings> AnalyzerSettings;

	if (Settings)
	{
		AnalyzerSettings = Settings->GetSettings(InSampleRate, InNumChannels);
	}

	return AnalyzerSettings;
}

static TArray<FLoudnessResults> ConvertToBlueprintResults(ULoudnessSettings* Settings, const TArray<Audio::FLoudnessEntry>& InLoudnessArray)
{
	// Helper function for "Sort" (higher priority sorts last).
	struct FCompareLoudnessResults
	{
		FORCEINLINE bool operator()(const FLoudnessResults& A, const FLoudnessResults& B) const
		{
			return A.TimeSeconds < B.TimeSeconds;
		}
	};

	TArray<FLoudnessResults> ResultsArray;

	float LoudnessRange = FMath::Max(Settings->ExpectedMaxLoudness - Settings->NoiseFloorDb, 0.001f);

	for (const Audio::FLoudnessEntry& LoudnessEntry : InLoudnessArray)
	{
		FLoudnessResults NewResults;
		NewResults.Loudness = LoudnessEntry.Loudness;
		NewResults.NormalizedLoudness = FMath::Clamp((LoudnessEntry.Loudness - Settings->NoiseFloorDb) / LoudnessRange, 0.0f, 1.0f);
		NewResults.PerceptualEnergy = LoudnessEntry.Energy;
		NewResults.TimeSeconds = LoudnessEntry.Timestamp;

		ResultsArray.Add(NewResults);
	}

	// Sort by priority (lowest priority first).
	ResultsArray.Sort(FCompareLoudnessResults());
	return MoveTemp(ResultsArray);
}

void ULoudnessAnalyzer::BroadcastResults()
{
	TUniquePtr<const Audio::FLoudnessResult> LoudnessResults = GetResults<Audio::FLoudnessResult>();

	int32 NumChannels = LoudnessResults->GetNumChannels();

	if (NumChannels > 0)
	{
		bool bIsOnOverallLoudnessResultsBound = OnOverallLoudnessResults.IsBound();
		bool bIsOnLatestOverallLoudnessResultsBound = OnLatestOverallLoudnessResults.IsBound();
		if (bIsOnOverallLoudnessResultsBound || bIsOnLatestOverallLoudnessResultsBound)
		{
			const TArray<Audio::FLoudnessEntry>& OverallLoudnessArray = LoudnessResults->GetLoudnessArray();
			if (OverallLoudnessArray.Num() > 0)
			{
				TArray<FLoudnessResults> Results = ConvertToBlueprintResults(Settings, OverallLoudnessArray);
				check(Results.Num() > 0);

				if (bIsOnOverallLoudnessResultsBound)
				{
					OnOverallLoudnessResults.Broadcast(Results);
				}
				if (bIsOnLatestOverallLoudnessResultsBound)
				{
					FLoudnessResults& Latest = Results[Results.Num() - 1];
					OnLatestOverallLoudnessResults.Broadcast(Latest);
				}
			}
		}

		bool bIsOnPerChannelLoudnessResultsBound = OnPerChannelLoudnessResults.IsBound();
		bool bIsOnLatestPerChannelLoudnessResultsBound = OnLatestPerChannelLoudnessResults.IsBound();

		if (bIsOnPerChannelLoudnessResultsBound || bIsOnLatestPerChannelLoudnessResultsBound)
		{
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
				const TArray<Audio::FLoudnessEntry>& LoudnessArray = LoudnessResults->GetChannelLoudnessArray(ChannelIndex);
				if (LoudnessArray.Num() > 0)
				{
					TArray<FLoudnessResults> Results = ConvertToBlueprintResults(Settings, LoudnessArray);
					check(Results.Num() > 0);

					if (bIsOnPerChannelLoudnessResultsBound)
					{
						OnPerChannelLoudnessResults.Broadcast(ChannelIndex, Results);
					}

					if (bIsOnLatestPerChannelLoudnessResultsBound)
					{
						OnLatestPerChannelLoudnessResults.Broadcast(ChannelIndex, Results[Results.Num() - 1]);
					}
				}
			}
		}
	}
}

FName ULoudnessAnalyzer::GetAnalyzerFactoryName() const
{
	static const FName FactoryName(TEXT("LoudnessFactory"));
	return FactoryName;
}
