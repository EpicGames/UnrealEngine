// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoudnessNRT.h"
#include "LoudnessNRTFactory.h"
#include "InterpolateSorted.h"
#include "AudioSynesthesiaLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LoudnessNRT)

namespace LoudnessNRTPrivate
{
	float InterpolateLoudness(TArrayView<const Audio::FLoudnessDatum> InLoudnessArray, const float InTimestamp)
	{
		int32 LowerIndex = INDEX_NONE;
		int32 UpperIndex = INDEX_NONE;
		float Alpha = 0.f;

		Audio::GetInterpolationParametersAtTimestamp(InLoudnessArray, InTimestamp, LowerIndex, UpperIndex, Alpha);

		if ((INDEX_NONE != LowerIndex) && (INDEX_NONE != UpperIndex))
		{
			return FMath::Lerp(InLoudnessArray[LowerIndex].Loudness, InLoudnessArray[UpperIndex].Loudness, Alpha);
		}

		return 0.f;
	}
}


/***************************************************************************/
/**********************    ULoudnessNRTSettings     ************************/
/***************************************************************************/

ULoudnessNRTSettings::ULoudnessNRTSettings()
:	AnalysisPeriod(0.01f)	
,	MinimumFrequency(20.f)
,	MaximumFrequency(20000.f)
,	CurveType(ELoudnessNRTCurveTypeEnum::D)
,	NoiseFloorDb(-60.f)
{}

TUniquePtr<Audio::IAnalyzerNRTSettings> ULoudnessNRTSettings::GetSettings(const float InSampleRate, const int32 InNumChannels) const
{
	TUniquePtr<Audio::FLoudnessNRTSettings> Settings = MakeUnique<Audio::FLoudnessNRTSettings>();

	Settings->AnalysisPeriod = AnalysisPeriod;
	Settings->MinAnalysisFrequency = MinimumFrequency;
	Settings->MaxAnalysisFrequency = MaximumFrequency;
	Settings->WindowType = Audio::EWindowType::Blackman;
	Settings->ScalingMethod = Audio::ELoudnessAnalyzerScalingMethod::BackCompatible;
		
	switch (CurveType)
	{
		case ELoudnessNRTCurveTypeEnum::A:
			Settings->LoudnessCurveType = Audio::ELoudnessCurveType::A;
			break;

		case ELoudnessNRTCurveTypeEnum::B:
			Settings->LoudnessCurveType = Audio::ELoudnessCurveType::B;
			break;
			
		case ELoudnessNRTCurveTypeEnum::C:
			Settings->LoudnessCurveType = Audio::ELoudnessCurveType::C;
			break;

		case ELoudnessNRTCurveTypeEnum::D:
			Settings->LoudnessCurveType = Audio::ELoudnessCurveType::D;
			break;

		case ELoudnessNRTCurveTypeEnum::K:
			Settings->LoudnessCurveType = Audio::ELoudnessCurveType::K;
			break;

		case ELoudnessNRTCurveTypeEnum::None:
		default:
			Settings->LoudnessCurveType = Audio::ELoudnessCurveType::None;
			break;
	}
	
	return Settings;
}

#if WITH_EDITOR
FText ULoudnessNRTSettings::GetAssetActionName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AssetSoundSynesthesiaLoudnessNRTSettings", "Synesthesia NRT Settings (Loudness)");
}

UClass* ULoudnessNRTSettings::GetSupportedClass() const
{
	return ULoudnessNRTSettings::StaticClass();
}
#endif

/***************************************************************************/
/**********************        ULoudnessNRT         ************************/
/***************************************************************************/

ULoudnessNRT::ULoudnessNRT()
{
	Settings = CreateDefaultSubobject<ULoudnessNRTSettings>(TEXT("DefaultLoudnessNRTSettings"));
#if WITH_EDITOR
	// Bind settings to audio analyze so changes to default settings will trigger analysis.
	SetSettingsDelegate(Settings);
#endif
}

void ULoudnessNRT::GetLoudnessAtTime(const float InSeconds, float& OutLoudness) const
{
	GetChannelLoudnessAtTime(InSeconds, Audio::FLoudnessNRTResult::ChannelIndexOverall, OutLoudness);
}

void ULoudnessNRT::GetChannelLoudnessAtTime(const float InSeconds, const int32 InChannel, float& OutLoudness) const
{
	OutLoudness = 0.0f;

	TSharedPtr<const Audio::FLoudnessNRTResult, ESPMode::ThreadSafe> LoudnessResult = GetResult<Audio::FLoudnessNRTResult>();

	if (LoudnessResult.IsValid())
	{
		// The loudness result should never used here if it is not already sorted.
		check(LoudnessResult->IsSortedChronologically());

		if (!LoudnessResult->ContainsChannel(InChannel))
		{
			UE_LOGF(LogAudioSynesthesia, Warning, "LoudnessNRT does not contain channel %d", InChannel);
			return;
		}

		const TArray<Audio::FLoudnessDatum>& LoudnessArray = LoudnessResult->GetChannelLoudnessArray(InChannel);

		OutLoudness = LoudnessNRTPrivate::InterpolateLoudness(LoudnessArray, InSeconds);
	}
}

void ULoudnessNRT::GetNormalizedLoudnessAtTime(const float InSeconds, float& OutLoudness) const
{
	GetNormalizedChannelLoudnessAtTime(InSeconds, Audio::FLoudnessNRTResult::ChannelIndexOverall, OutLoudness);
}

void ULoudnessNRT::GetNormalizedChannelLoudnessAtTime(const float InSeconds, const int32 InChannel, float& OutLoudness) const
{
	OutLoudness = 0.0f;

	TSharedPtr<const Audio::FLoudnessNRTResult, ESPMode::ThreadSafe> LoudnessResult = GetResult<Audio::FLoudnessNRTResult>();

	if (LoudnessResult.IsValid())
	{
		// The loudness result should never used here if it is not already sorted.
		check(LoudnessResult->IsSortedChronologically());

		if (!LoudnessResult->ContainsChannel(InChannel))
		{
			UE_LOGF(LogAudioSynesthesia, Warning, "LoudnessNRT does not contain channel %d", InChannel);
			return;
		}

		const TArray<Audio::FLoudnessDatum>& LoudnessArray = LoudnessResult->GetChannelLoudnessArray(InChannel);

		OutLoudness = LoudnessNRTPrivate::InterpolateLoudness(LoudnessArray, InSeconds);

		// Subtract offset
		OutLoudness -= Settings->NoiseFloorDb;

		// Divide by range
		OutLoudness /= FMath::Max(LoudnessResult->GetChannelLoudnessRange(InChannel, Settings->NoiseFloorDb), SMALL_NUMBER);

		// Clamp to desired range in case value was below noise floor
		OutLoudness = FMath::Clamp(OutLoudness, 0.f, 1.f);
	}
}

TUniquePtr<Audio::IAnalyzerNRTSettings> ULoudnessNRT::GetSettings(const float InSampleRate, const int32 InNumChannels) const
{
	TUniquePtr<Audio::IAnalyzerNRTSettings> AnalyzerSettings;

	if (Settings)
	{
		AnalyzerSettings = Settings->GetSettings(InSampleRate, InNumChannels);	
	}

	return AnalyzerSettings;
}

#if WITH_EDITOR
FText ULoudnessNRT::GetAssetActionName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AssetSoundSynesthesiaLoudnessNRT", "Synesthesia NRT (Loudness)");
}

UClass* ULoudnessNRT::GetSupportedClass() const
{
	return ULoudnessNRT::StaticClass();
}

bool ULoudnessNRT::ShouldEventTriggerAnalysis(FPropertyChangedEvent& PropertyChangeEvent)
{
	return Settings != nullptr && Super::ShouldEventTriggerAnalysis(PropertyChangeEvent);
}
#endif // WITH_EDITOR

FName ULoudnessNRT::GetAnalyzerNRTFactoryName() const
{
	static const FName FactoryName(TEXT("LoudnessNRTFactory"));
	return FactoryName;
}


