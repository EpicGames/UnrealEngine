// Copyright Epic Games, Inc. All Rights Reserved.

#include "Meter.h"
#include "MeterFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Meter)

TUniquePtr<Audio::IAnalyzerSettings> UMeterSettings::GetSettings(const int32 InSampleRate, const int32 InNumChannels) const
{
	TUniquePtr<Audio::FMeterSettings> Settings = MakeUnique<Audio::FMeterSettings>();

	Settings->AnalysisPeriod = AnalysisPeriod;

	switch (PeakMode)
	{
	case EMeterPeakType::MeanSquared:
		Settings->MeterPeakMode = Audio::EPeakMode::MeanSquared;
		break;

	case EMeterPeakType::RootMeanSquared:
		Settings->MeterPeakMode = Audio::EPeakMode::RootMeanSquared;
		break;

	case EMeterPeakType::TruePeak:
		Settings->MeterPeakMode = Audio::EPeakMode::TruePeak;
		break;

	default:
	case EMeterPeakType::Peak:
		Settings->MeterPeakMode = Audio::EPeakMode::Peak;
		break;
	}

	Settings->MeterAttackTime = MeterAttackTime;
	Settings->MeterReleaseTime = MeterReleaseTime;
	Settings->PeakHoldTime = PeakHoldTime;
	Settings->ClippingThreshold = ClippingThreshold;
	Settings->bIsAnalog = bIsAnalog;

	return Settings;
}

#if WITH_EDITOR
FText UMeterSettings::GetAssetActionName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AssetSoundSynesthesiaMeterSettings", "Synesthesia Real-Time Settings (Meter)");
}

UClass* UMeterSettings::GetSupportedClass() const
{
	return UMeterSettings::StaticClass();
}
#endif

UMeterAnalyzer::UMeterAnalyzer()
{
	Settings = CreateDefaultSubobject<UMeterSettings>(TEXT("DefaultMeterSettings"));
}

TUniquePtr<Audio::IAnalyzerSettings> UMeterAnalyzer::GetSettings(const int32 InSampleRate, const int32 InNumChannels) const
{
	TUniquePtr<Audio::IAnalyzerSettings> AnalyzerSettings;

	if (Settings)
	{
		AnalyzerSettings = Settings->GetSettings(InSampleRate, InNumChannels);
	}

	return AnalyzerSettings;
}

static TArray<FMeterResults> ConvertToBlueprintResults(UMeterSettings* Settings, const TArray<Audio::FMeterEntry>& InMeterArray)
{
	TArray<FMeterResults> ResultsArray;

	for (const Audio::FMeterEntry& MeterEntry : InMeterArray)
	{
		FMeterResults NewResults;

		NewResults.MeterValue = Audio::ConvertToDecibels(MeterEntry.MeterValue);
		NewResults.PeakValue = Audio::ConvertToDecibels(MeterEntry.PeakValue);
		NewResults.ClippingValue = Audio::ConvertToDecibels(MeterEntry.ClippingValue);
		NewResults.NumSamplesClipping = MeterEntry.NumSamplesClipping;
		NewResults.TimeSeconds = MeterEntry.Timestamp;

		ResultsArray.Add(NewResults);
	}

	// Sort by priority (lowest priority first).
	ResultsArray.Sort([](const FMeterResults& A, const FMeterResults& B) 		
	{ 
		return A.TimeSeconds < B.TimeSeconds;
	});

	return MoveTemp(ResultsArray);
}

void UMeterAnalyzer::BroadcastResults()
{
	TUniquePtr<const Audio::FMeterResult> MeterResults = GetResults<Audio::FMeterResult>();
	if (!MeterResults.IsValid())
	{
		return;
	}

	int32 NumChannels = MeterResults->GetNumChannels();

	if (NumChannels > 0)
	{
		bool bIsOnOverallMeterResultsBound = OnOverallMeterResults.IsBound() || OnOverallMeterResultsNative.IsBound();
		bool bIsOnLatestOverallMeterResultsBound = OnLatestOverallMeterResults.IsBound() || OnLatestOverallMeterResultsNative.IsBound();
		if (bIsOnOverallMeterResultsBound || bIsOnLatestOverallMeterResultsBound)
		{
			const TArray<Audio::FMeterEntry>& OverallMeterArray = MeterResults->GetMeterArray();
			if (OverallMeterArray.Num() > 0)
			{
				TArray<FMeterResults> Results = ConvertToBlueprintResults(Settings, OverallMeterArray);
				check(Results.Num() > 0);

				OnOverallMeterResults.Broadcast(Results);
				OnOverallMeterResultsNative.Broadcast(this, Results);

				FMeterResults& Latest = Results[Results.Num() - 1];
				OnLatestOverallMeterResults.Broadcast(Latest);
				OnLatestOverallMeterResultsNative.Broadcast(this, Latest);

			}
		}

		bool bIsOnPerChannelMeterResultsBound = OnPerChannelMeterResults.IsBound() || OnPerChannelMeterResultsNative.IsBound();
		bool bIsOnLatestPerChannelMeterResultsBound = OnLatestPerChannelMeterResults.IsBound() || OnLatestPerChannelMeterResultsNative.IsBound();

		if (bIsOnPerChannelMeterResultsBound || bIsOnLatestPerChannelMeterResultsBound)
		{
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
				const TArray<Audio::FMeterEntry>& MeterArray = MeterResults->GetChannelMeterArray(ChannelIndex);
				if (MeterArray.Num() > 0)
				{
					TArray<FMeterResults> Results = ConvertToBlueprintResults(Settings, MeterArray);
					check(Results.Num() > 0);

					OnPerChannelMeterResults.Broadcast(ChannelIndex, Results);
					OnPerChannelMeterResultsNative.Broadcast(this, ChannelIndex, Results);

					OnLatestPerChannelMeterResults.Broadcast(ChannelIndex, Results[Results.Num() - 1]);
					OnLatestPerChannelMeterResultsNative.Broadcast(this, ChannelIndex, Results[Results.Num() - 1]);
				}
			}
		}
	}
}

FName UMeterAnalyzer::GetAnalyzerFactoryName() const
{
	static const FName FactoryName(TEXT("MeterFactory"));
	return FactoryName;
}
