// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnsetNRT.h"
#include "OnsetNRTFactory.h"
#include "AudioSynesthesiaLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnsetNRT)

namespace
{
	// Looks up onsets that fall between two timestamps and adds there info to the output arrays. 
	void AddOnsetsInRange(const Audio::FOnsetNRTResult& OnsetResult, const float InStartSeconds, const float InEndSeconds, const int32 InChannel, TArray<float>& OutOnsetTimestamps, TArray<float>& OutOnsetStrengths)
	{
		// Onset results should always be sorted temporally within UOnsetNRT objects
		check(OnsetResult.IsSortedChronologically());

		const TArray<Audio::FOnset>& OnsetArray = OnsetResult.GetOnsetsForChannel(InChannel);

		// By design we are not inclusive of InStartSeconds but are inclusive with the InEndSeconds timestamp. 
		// This is to avoid returning duplicate onsets when this function is called repeatedly with incremented
		// timestamps.
		int32 StartTimestampIndex = Algo::UpperBoundBy(OnsetArray, InStartSeconds, [](const Audio::FOnset& Onset) { return Onset.Timestamp; });

		int32 EndTimestampIndex = Algo::UpperBoundBy(OnsetArray, InEndSeconds, [](const Audio::FOnset& Onset) { return Onset.Timestamp; });
		

		// Add to output arrays.
		for (int32 i = StartTimestampIndex; i < EndTimestampIndex; i++)
		{
			const Audio::FOnset& Onset = OnsetArray[i];
			OutOnsetTimestamps.Add(Onset.Timestamp);
			OutOnsetStrengths.Add(Onset.Strength);
		}
	}
}

/*************************************************************************************/
/***************************** UOnsetNRTSettings *************************************/
/*************************************************************************************/

// Construct default settings.
UOnsetNRTSettings::UOnsetNRTSettings()
:	bDownmixToMono(false)
,	GranularityInSeconds(0.01f)
,	Sensitivity(0.5f)
,	MinimumFrequency(20.f)
,	MaximumFrequency(16000.f)
{}

TUniquePtr<Audio::IAnalyzerNRTSettings> UOnsetNRTSettings::GetSettings(const float InSampleRate, const int32 InNumChannels) const
{
	// Make a default FOnsetNRTSettings object
	TUniquePtr<Audio::FOnsetNRTSettings> OnsetNRTSettings = MakeUnique<Audio::FOnsetNRTSettings>();

	// Apply global settings
	OnsetNRTSettings->bDownmixToMono = bDownmixToMono;

	// Apply Onset strength settings
	// Note: Granularity effects hop size, window size and fft size.
	OnsetNRTSettings->OnsetStrengthSettings.NumHopFrames = FMath::Max(FMath::RoundToInt(GranularityInSeconds * InSampleRate), 1);
	OnsetNRTSettings->OnsetStrengthSettings.NumWindowFrames = 4 * OnsetNRTSettings->OnsetStrengthSettings.NumHopFrames;
	OnsetNRTSettings->OnsetStrengthSettings.ComparisonLag = 4;
	OnsetNRTSettings->OnsetStrengthSettings.FFTSize = FMath::RoundUpToPowerOfTwo(OnsetNRTSettings->OnsetStrengthSettings.NumWindowFrames);
	OnsetNRTSettings->OnsetStrengthSettings.WindowType = Audio::EWindowType::Blackman;

	OnsetNRTSettings->OnsetStrengthSettings.NoiseFloorDb = -60; 
	// Onset strength mel band settings
	OnsetNRTSettings->OnsetStrengthSettings.MelSettings.NumBands = 26;
	OnsetNRTSettings->OnsetStrengthSettings.MelSettings.KernelMinCenterFreq = FMath::Max(MinimumFrequency * 1.1f, 20.f);
	OnsetNRTSettings->OnsetStrengthSettings.MelSettings.KernelMaxCenterFreq = FMath::Min(MaximumFrequency * 0.9f, 20000.f);
	OnsetNRTSettings->OnsetStrengthSettings.MelSettings.Normalization = Audio::EMelNormalization::EqualEnergy;
	OnsetNRTSettings->OnsetStrengthSettings.MelSettings.BandWidthStretch = 1.f;

	// Peak picker settings
	// 
	// The "Sensitivity" variable has a strong influence on the peak picker settings. 
	// A higher sensitivity results in many more peaks being picked, and a lower sensitivity
	// results in only the largest peaks being picked. 
	
	// Peak picker is picking peaks from analyzed audio windows. So we need to convert timestamps 
	// to window indices.
	const float HopRate = InSampleRate / static_cast<float>(OnsetNRTSettings->OnsetStrengthSettings.NumHopFrames);
	TFunction<int32 (float)> TimeToWindow = [HopRate](float InTime) 
	{
		const int32 MinSample = 1;
		
		return FMath::Max(MinSample, FMath::RoundToInt(InTime * HopRate));
	};

	// Flip sensitivity when doing interpolation. An alpha == 1 is not sensitive. An alpha == 0 is most sensitive.
	const float Alpha = 1.f - Sensitivity;

	static const float MinPreMaxSeconds = 0.01f;
	static const float MaxPreMaxSeconds = 0.3f;
	OnsetNRTSettings->PeakPickerSettings.NumPreMax = TimeToWindow(FMath::InterpExpoIn(MinPreMaxSeconds, MaxPreMaxSeconds, Alpha));

	static const float MinPostMaxSeconds = 0.01f;
	static const float MaxPostMaxSeconds = 0.1f;
	OnsetNRTSettings->PeakPickerSettings.NumPostMax = TimeToWindow(FMath::InterpExpoIn(MinPostMaxSeconds, MaxPostMaxSeconds, Alpha));

	static const float MinPreMeanSeconds = 0.05f;
	static const float MaxPreMeanSeconds = 0.5f;
	OnsetNRTSettings->PeakPickerSettings.NumPreMean = TimeToWindow(FMath::InterpExpoIn(MinPreMeanSeconds, MaxPreMeanSeconds, Alpha));

	static const float MinPostMeanSeconds = 0.05f;
	static const float MaxPostMeanSeconds = 0.5f;
	OnsetNRTSettings->PeakPickerSettings.NumPostMean = TimeToWindow(FMath::InterpExpoIn(MinPostMeanSeconds, MaxPostMeanSeconds, Alpha));

	static const float MinWaitSeconds = 0.01f;
	static const float MaxWaitSeconds = 0.25f;
	OnsetNRTSettings->PeakPickerSettings.NumWait = TimeToWindow(FMath::InterpExpoIn(MinWaitSeconds, MaxWaitSeconds, Alpha));

	static const float MinMeanDelta = 0.01f;
	static const float MaxMeanDelta = 0.2f;
	OnsetNRTSettings->PeakPickerSettings.MeanDelta = FMath::InterpExpoIn(MinMeanDelta, MaxMeanDelta, Alpha);

	return OnsetNRTSettings;

}

#if WITH_EDITOR
FText UOnsetNRTSettings::GetAssetActionName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AssetSoundSynesthesiaOnsetNRTSettings", "Synesthesia NRT Settings (Onset)");
}

UClass* UOnsetNRTSettings::GetSupportedClass() const
{
	return UOnsetNRTSettings::StaticClass();
}
#endif
/*************************************************************************************/
/********************************* UOnsetNRT *****************************************/
/*************************************************************************************/

UOnsetNRT::UOnsetNRT()
{
	Settings = CreateDefaultSubobject<UOnsetNRTSettings>(TEXT("DefaultOnsetNRTSettings"));

#if WITH_EDITOR
	// Bind settings to audio analyze so changes to default settings will trigger analysis.
	SetSettingsDelegate(Settings);
#endif
}


void UOnsetNRT::GetChannelOnsetsBetweenTimes(const float InStartSeconds, const float InEndSeconds, const int32 InChannel, TArray<float>& OutOnsetTimestamps, TArray<float>& OutOnsetStrengths) const
{
	// Clear output arrays
	OutOnsetTimestamps.Reset();
	OutOnsetStrengths.Reset();

	TSharedPtr<const Audio::FOnsetNRTResult, ESPMode::ThreadSafe> OnsetResult = GetResult<Audio::FOnsetNRTResult>();

	if (OnsetResult.IsValid())
	{
		if (!OnsetResult->ContainsChannel(InChannel))
		{
			// invalid channel index.
			UE_LOGF(LogAudioSynesthesia, Warning, "OnsetNRT does not contain channel %d", InChannel);
			return;
		}

		AddOnsetsInRange(*OnsetResult, InStartSeconds, InEndSeconds, InChannel, OutOnsetTimestamps, OutOnsetStrengths);
	}
}

void UOnsetNRT::GetNormalizedChannelOnsetsBetweenTimes(const float InStartSeconds, const float InEndSeconds, const int32 InChannel, TArray<float>& OutOnsetTimestamps, TArray<float>& OutOnsetStrengths) const
{
	// Clear output arrays
	OutOnsetTimestamps.Reset();
	OutOnsetStrengths.Reset();

	TSharedPtr<const Audio::FOnsetNRTResult, ESPMode::ThreadSafe> OnsetResult = GetResult<Audio::FOnsetNRTResult>();

	if (OnsetResult.IsValid())
	{
		if (!OnsetResult->ContainsChannel(InChannel))
		{
			// Invalid channel index.
			UE_LOGF(LogAudioSynesthesia, Warning, "OnsetNRT does not contain channel %d", InChannel);
			return;
		}

		AddOnsetsInRange(*OnsetResult, InStartSeconds, InEndSeconds, InChannel, OutOnsetTimestamps, OutOnsetStrengths);

		// Normalize by maximum onset. Minimum onset value is assumed to be zero.
		const FFloatInterval OnsetStrengthInterval = OnsetResult->GetChannelOnsetInterval(InChannel);
		
		if (OnsetStrengthInterval.Max > 0.f)
		{
			for (float& Strength : OutOnsetStrengths)
			{
				Strength /= OnsetStrengthInterval.Max;
			}
		}
	}
}

TUniquePtr<Audio::IAnalyzerNRTSettings> UOnsetNRT::GetSettings(const float InSampleRate, const int32 InNumChannels) const
{
	TUniquePtr<Audio::IAnalyzerNRTSettings> AnalyzerSettings;

	if (Settings)
	{
		AnalyzerSettings = Settings->GetSettings(InSampleRate, InNumChannels);	
	}

	return AnalyzerSettings;
}

#if WITH_EDITOR
FText UOnsetNRT::GetAssetActionName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AssetSoundSynesthesiaOnset", "Synesthesia NRT (Onset)");
}

UClass* UOnsetNRT::GetSupportedClass() const
{
	return UOnsetNRT::StaticClass();
}

bool UOnsetNRT::ShouldEventTriggerAnalysis(FPropertyChangedEvent& PropertyChangeEvent)
{
	return Settings != nullptr && Super::ShouldEventTriggerAnalysis(PropertyChangeEvent);
}
#endif // WITH_EDITOR

FName UOnsetNRT::GetAnalyzerNRTFactoryName() const
{
	static const FName FactoryName(TEXT("OnsetNRTFactory"));
	return FactoryName;
}


