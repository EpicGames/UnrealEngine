// Copyright Epic Games, Inc. All Rights Reserved.

#include "SynesthesiaSpectrumAnalysis.h"
#include "SynesthesiaSpectrumAnalysisFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SynesthesiaSpectrumAnalysis)

namespace 
{
	// Convert EFFTSize to int32
	const int32 BPFFTSizeEnumToInt(EFFTSize InFFTSizeEnum)
	{
		switch (InFFTSizeEnum)
		{
		case EFFTSize::Min:
			return 64;
		case EFFTSize::Small:
			return 256;
		case EFFTSize::Medium:
			return 512;
		case EFFTSize::Large:
			return 1024;
		case EFFTSize::VeryLarge:
			return 2048;
		case EFFTSize::Max:
			return 4096;

		case EFFTSize::DefaultSize:
		default:
			return 512;
		};
	}

	// EAudioSpectrumType to Audio::ESynesthesiaSpectrumType
	const Audio::ESynesthesiaSpectrumType BPSpectrumTypeToSynesthesiaSpectrumType(EAudioSpectrumType InSpectrumType)
	{
		switch (InSpectrumType)
		{
		case EAudioSpectrumType::MagnitudeSpectrum:
			return Audio::ESynesthesiaSpectrumType::MagnitudeSpectrum;

		case EAudioSpectrumType::Decibel:
			return Audio::ESynesthesiaSpectrumType::Decibel;

		case EAudioSpectrumType::PowerSpectrum:
		default:
			return Audio::ESynesthesiaSpectrumType::PowerSpectrum;
		};
	}

	// EFFTWindowType to Audio::ESynesthesiaFFTWindowType
	const Audio::EWindowType BPWindowTypeToWindowType(EFFTWindowType InSpectrumType)
	{
		switch (InSpectrumType)
		{
		case EFFTWindowType::Hamming:
			return Audio::EWindowType::Hamming;

		case EFFTWindowType::Hann:
			return Audio::EWindowType::Hann;

		case EFFTWindowType::Blackman:
			return Audio::EWindowType::Blackman;

		case EFFTWindowType::None:
		default:
			return Audio::EWindowType::None;
		};
	}
}

TUniquePtr<Audio::IAnalyzerSettings> USynesthesiaSpectrumAnalysisSettings::GetSettings(const int32 InSampleRate, const int32 InNumChannels) const
{
	TUniquePtr<Audio::FSynesthesiaSpectrumAnalysisSettings> Settings = MakeUnique<Audio::FSynesthesiaSpectrumAnalysisSettings>();

	Settings->AnalysisPeriod = AnalysisPeriod;
	Settings->FFTSize = BPFFTSizeEnumToInt(FFTSize);
	Settings->SpectrumType = BPSpectrumTypeToSynesthesiaSpectrumType(SpectrumType);
	Settings->WindowType = BPWindowTypeToWindowType(WindowType);
	Settings->bDownmixToMono = bDownmixToMono;
	return Settings;
}

#if WITH_EDITOR
FText USynesthesiaSpectrumAnalysisSettings::GetAssetActionName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AssetSoundSynesthesiaSpectrumSettings", "Synesthesia Real-Time Settings (Spectrum)");
}

UClass* USynesthesiaSpectrumAnalysisSettings::GetSupportedClass() const
{
	return USynesthesiaSpectrumAnalysisSettings::StaticClass();
}
#endif

USynesthesiaSpectrumAnalyzer::USynesthesiaSpectrumAnalyzer()
{
	Settings = CreateDefaultSubobject<USynesthesiaSpectrumAnalysisSettings>(TEXT("DefaultSynesthesiaSpectrumSettings"));
}

TUniquePtr<Audio::IAnalyzerSettings> USynesthesiaSpectrumAnalyzer::GetSettings(const int32 InSampleRate, const int32 InNumChannels) const
{
	TUniquePtr<Audio::IAnalyzerSettings> AnalyzerSettings;

	if (Settings)
	{
		AnalyzerSettings = Settings->GetSettings(InSampleRate, InNumChannels);
	}

	return AnalyzerSettings;
}

static TArray<FSynesthesiaSpectrumResults> ConvertToBlueprintResults(USynesthesiaSpectrumAnalysisSettings* Settings, const TArray<Audio::FSynesthesiaSpectrumEntry>& InSynesthesiaSpectrumArray)
{
	TArray<FSynesthesiaSpectrumResults> ResultsArray;

	for (const Audio::FSynesthesiaSpectrumEntry& SynesthesiaSpectrumEntry : InSynesthesiaSpectrumArray)
	{
		FSynesthesiaSpectrumResults NewResults;

		NewResults.SpectrumValues = SynesthesiaSpectrumEntry.SpectrumValues;
		NewResults.TimeSeconds = SynesthesiaSpectrumEntry.Timestamp;

		ResultsArray.Add(NewResults);
	}

	// Sort by time priority (lowest priority/latest first).
	ResultsArray.Sort([](const FSynesthesiaSpectrumResults& A, const FSynesthesiaSpectrumResults& B) 		
	{ 
		return A.TimeSeconds < B.TimeSeconds;
	});

	return MoveTemp(ResultsArray);
}

void USynesthesiaSpectrumAnalyzer::BroadcastResults()
{
	TUniquePtr<const Audio::FSynesthesiaSpectrumResult> SynesthesiaSpectrumResults = GetResults<Audio::FSynesthesiaSpectrumResult>();
	if (!SynesthesiaSpectrumResults.IsValid())
	{
		return;
	}

	int32 NumChannels = SynesthesiaSpectrumResults->GetNumChannels();

	if (NumChannels <= 0)
	{
		return;
	}

	bool bIsOnSpectrumResultsBound = OnSpectrumResults.IsBound() || OnSpectrumResultsNative.IsBound();
	bool bIsOnLatestSpectrumResultsBound = OnLatestSpectrumResults.IsBound() || OnLatestSpectrumResultsNative.IsBound();

	if (bIsOnSpectrumResultsBound || bIsOnLatestSpectrumResultsBound)
	{
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
		{
			const TArray<Audio::FSynesthesiaSpectrumEntry>& SynesthesiaSpectrumArray = SynesthesiaSpectrumResults->GetChannelSpectrumArray(ChannelIndex);
			if (SynesthesiaSpectrumArray.Num() > 0)
			{
				// Sorts results by timestamp
				TArray<FSynesthesiaSpectrumResults> Results = ConvertToBlueprintResults(Settings, SynesthesiaSpectrumArray);
				check(Results.Num() > 0);

				OnSpectrumResults.Broadcast(ChannelIndex, Results);
				OnSpectrumResultsNative.Broadcast(this, ChannelIndex, Results);

				OnLatestSpectrumResults.Broadcast(ChannelIndex, Results[Results.Num() - 1]);
				OnLatestSpectrumResultsNative.Broadcast(this, ChannelIndex, Results[Results.Num() - 1]);
			}
		}
	}
}

void USynesthesiaSpectrumAnalyzer::GetCenterFrequencies(const float InSampleRate, TArray<float>& OutCenterFrequencies)
{
	const int32 NumFrequencies = GetNumCenterFrequencies();
	if (NumFrequencies == OutCenterFrequencies.Num())
	{
		// Calculate frequencies 0 to Nyquist inclusive
		float BinSize = InSampleRate / BPFFTSizeEnumToInt(Settings->FFTSize);
		for (int i = 0; i < NumFrequencies; ++i)
		{
			OutCenterFrequencies[i] = i * BinSize;
		}
	}
}

const int32 USynesthesiaSpectrumAnalyzer::GetNumCenterFrequencies() const
{
	return BPFFTSizeEnumToInt(Settings->FFTSize) / 2 + 1;
}

FName USynesthesiaSpectrumAnalyzer::GetAnalyzerFactoryName() const
{
	static const FName FactoryName(TEXT("SpectrumAnalysisFactory"));
	return FactoryName;
}
