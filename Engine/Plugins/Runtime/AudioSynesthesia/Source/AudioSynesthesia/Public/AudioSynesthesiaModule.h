// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "LoudnessNRTFactory.h"
#include "ConstantQNRTFactory.h"
#include "OnsetNRTFactory.h"

#define UE_API AUDIOSYNESTHESIA_API

namespace Audio
{
	class FAudioSynesthesiaModule : public IModuleInterface
	{
	public:

		// IModuleInterface interface
		UE_API virtual void StartupModule() override;
		UE_API virtual void ShutdownModule() override;

	private:
		FLoudnessNRTFactory LoudnessFactory;
		FConstantQNRTFactory ConstantQFactory;
		FOnsetNRTFactory OnsetFactory;
	};
}

#undef UE_API
