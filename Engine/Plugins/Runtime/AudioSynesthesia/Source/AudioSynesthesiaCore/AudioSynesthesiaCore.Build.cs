// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class AudioSynesthesiaCore : ModuleRules
	{
        public AudioSynesthesiaCore(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePathModuleNames.AddRange(
				new string[] {
					"AudioAnalyzer"
				}
			);
			
            PublicDependencyModuleNames.AddRange(
				new string[] {
                    "Core",
					"CoreUObject",
					"SignalProcessing",
                }
            );

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AudioMixerCore"
				}
			);
		}
	}
}
