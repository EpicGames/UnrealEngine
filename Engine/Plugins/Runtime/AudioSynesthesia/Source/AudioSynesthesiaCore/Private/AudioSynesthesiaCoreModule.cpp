// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioSynesthesiaCoreModule.h"

#include "AudioSynesthesiaCoreLog.h"
#include "ConstantQFactory.h"
#include "ConstantQNRTFactory.h"
#include "Features/IModularFeatures.h"
#include "HAL/LowLevelMemTracker.h"
#include "LoudnessFactory.h"
#include "LoudnessNRTFactory.h"
#include "LKFSFactory.h"
#include "LKFSNRTFactory.h"
#include "MeterFactory.h"
#include "Modules/ModuleManager.h"
#include "OnsetNRTFactory.h"
#include "SynesthesiaSpectrumAnalysisFactory.h"

DEFINE_LOG_CATEGORY(LogAudioSynesthesiaCore);

namespace Audio
{
	class FAudioSynesthesiaCoreModule : public IAudioSynesthesiaCoreModule
	{
		public:
			void StartupModule()
			{
				LLM_SCOPE_BYNAME(TEXT("Audio/AudioAnalysis"));
				FModuleManager::Get().LoadModuleChecked(TEXT("SignalProcessing"));

				// Register factories on startup
				IModularFeatures::Get().RegisterModularFeature(FLoudnessNRTFactory::GetModularFeatureName(), &LoudnessNRTFactory);
				IModularFeatures::Get().RegisterModularFeature(FLKFSNRTFactory::GetModularFeatureName(), &LKFSNRTFactory);
				IModularFeatures::Get().RegisterModularFeature(FConstantQNRTFactory::GetModularFeatureName(), &ConstantQNRTFactory);
				IModularFeatures::Get().RegisterModularFeature(FOnsetNRTFactory::GetModularFeatureName(), &OnsetNRTFactory);

				IModularFeatures::Get().RegisterModularFeature(FLoudnessFactory::GetModularFeatureName(), &LoudnessFactory);
				IModularFeatures::Get().RegisterModularFeature(FConstantQFactory::GetModularFeatureName(), &ConstantQFactory);
				IModularFeatures::Get().RegisterModularFeature(FMeterFactory::GetModularFeatureName(), &MeterFactory);
				IModularFeatures::Get().RegisterModularFeature(FLKFSFactory::GetModularFeatureName(), &LKFSFactory);
				IModularFeatures::Get().RegisterModularFeature(FSynesthesiaSpectrumAnalysisFactory::GetModularFeatureName(), &SpectralAnalysisFactory);
			}

			void ShutdownModule()
			{
				LLM_SCOPE_BYNAME(TEXT("Audio/AudioAnalysis"));
				// Unregister factories on shutdown
				IModularFeatures::Get().UnregisterModularFeature(FLoudnessNRTFactory::GetModularFeatureName(), &LoudnessNRTFactory);
				IModularFeatures::Get().UnregisterModularFeature(FLKFSNRTFactory::GetModularFeatureName(), &LKFSNRTFactory);
				IModularFeatures::Get().UnregisterModularFeature(FConstantQNRTFactory::GetModularFeatureName(), &ConstantQNRTFactory);
				IModularFeatures::Get().UnregisterModularFeature(FOnsetNRTFactory::GetModularFeatureName(), &OnsetNRTFactory);

				IModularFeatures::Get().UnregisterModularFeature(FLoudnessFactory::GetModularFeatureName(), &LoudnessFactory);
				IModularFeatures::Get().UnregisterModularFeature(FConstantQFactory::GetModularFeatureName(), &ConstantQFactory);
				IModularFeatures::Get().UnregisterModularFeature(FMeterFactory::GetModularFeatureName(), &MeterFactory);
				IModularFeatures::Get().UnregisterModularFeature(FLKFSFactory::GetModularFeatureName(), &LKFSFactory);
				IModularFeatures::Get().UnregisterModularFeature(FSynesthesiaSpectrumAnalysisFactory::GetModularFeatureName(), &SpectralAnalysisFactory);
			}

		private:
			FLoudnessNRTFactory LoudnessNRTFactory;
			FLKFSNRTFactory LKFSNRTFactory;
			FConstantQNRTFactory ConstantQNRTFactory;
			FOnsetNRTFactory OnsetNRTFactory;

			FLoudnessFactory LoudnessFactory;
			FConstantQFactory ConstantQFactory;
			FMeterFactory MeterFactory; 
			FLKFSFactory LKFSFactory; 
			FSynesthesiaSpectrumAnalysisFactory SpectralAnalysisFactory;
	};
}

IMPLEMENT_MODULE(Audio::FAudioSynesthesiaCoreModule, AudioSynesthesiaCore);

