// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSynesthesiaCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FAudioSynesthesiaCustomVersion::GUID( 0x82E77C4E, 0x332343A5, 0xB46B13C5, 0x97310DF3 );

// Register the custom version with core
FCustomVersionRegistration GRegisterAudioSynesthesiaCustomVersion( FAudioSynesthesiaCustomVersion::GUID, FAudioSynesthesiaCustomVersion::LatestVersion, TEXT( "AudioSynesthesiaVer" ) );
