// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoCorrelationPitchDetector.h"

#include "AudioSynesthesiaCoreLog.h"
#include "DSP/Dsp.h"
#include "PeakPicker.h"
#include "DSP/BlockCorrelator.h"

namespace Audio
{

	namespace AutoCorrelationPitchDetectorIntrinsics
	{
		constexpr int32 MinimumWindowSize = 512;
		constexpr int32 MaximumWindowSize = 8192;
		constexpr int32 PeakPickerMinSize = 2;

		// Remap input between 0 and 1 to a space result between the minimum and maximum.
		// @param InValue - value between 0 and 1 (inclusive)
		// @param MinimumValue - value greater than or equal to 2.
		// @param MaximumValue - value greater than or equal to 2.
		//
		// @return Remapped value.
		int32 PowerScalePeakPickerValue(float InValue, int32 MinimumValue, int32 MaximumValue)
		{

			MinimumValue = FMath::Max(2, MinimumValue);
			MaximumValue = FMath::Max(2, MaximumValue);
			InValue = FMath::Clamp(InValue, 0.f, 1.f);

			float Power = FMath::Max(SMALL_NUMBER, FMath::Loge(static_cast<float>(MaximumValue)) / FMath::Loge(static_cast<float>(MinimumValue)));
			float Denom = FMath::Max(SMALL_NUMBER, Power - 1.f);

			// Power mapped scale between 0 and 1.
			const float Scale = (FMath::Pow(Power, InValue) - 1.f) / Denom;

			float Range = static_cast<float>(MaximumValue - MinimumValue);

			return FMath::Clamp(MinimumValue + FMath::RoundToInt(Range * Scale), MinimumValue, MaximumValue);
		}
	}

	FAutoCorrelationPitchDetector::FAutoCorrelationPitchDetector(const FAutoCorrelationPitchDetectorSettings& InSettings, float InSampleRate)
	:	Settings(InSettings)
	,	SampleRate(InSampleRate)
	,	MinAutoCorrBin(0)
	,	MaxAutoCorrBin(0)
	,	WindowCounter(0)
	,	SlidingBuffer(1, 1)
	{
		using namespace AutoCorrelationPitchDetectorIntrinsics;

		if (!ensure(SampleRate > 0.f))
		{
			SampleRate = 1.f;
		}

		Settings.MinimumFrequency = FMath::Max(Settings.MinimumFrequency, 1.f);
		Settings.MaximumFrequency = FMath::Max(Settings.MaximumFrequency, 1.f);
		
		// setup sliding window
		int32 MinFreqNumSamples = FMath::RoundToInt(3.f * SampleRate / Settings.MinimumFrequency);
		MinFreqNumSamples = FMath::Max(MinFreqNumSamples, MinimumWindowSize);

		if (MinFreqNumSamples > MaximumWindowSize)
		{
			UE_LOGF(LogAudioSynesthesiaCore, Warning, "Pitches at minimum frequency (%f hz) may not be detected due to limit of internal window size", Settings.MinimumFrequency);
			MinFreqNumSamples = MaximumWindowSize;
		}

		int32 Log2WindowNumSamples = 1;
		while ((1 << Log2WindowNumSamples) < MinFreqNumSamples)
		{
			Log2WindowNumSamples++;
		}

		int32 HopNumSamples = FMath::Max(1, FMath::RoundToInt(SampleRate * Settings.AnalysisHopSeconds));

		int32 WindowNumSamples = 1 << Log2WindowNumSamples;

		SlidingBuffer = TSlidingBuffer<float>(WindowNumSamples, HopNumSamples);
		WindowBuffer.AddUninitialized(WindowNumSamples);

		// Setup range of result search
		MinAutoCorrBin = FMath::Clamp(FMath::FloorToInt(SampleRate / Settings.MaximumFrequency), 1, WindowNumSamples);
		MaxAutoCorrBin = FMath::Clamp(FMath::CeilToInt(SampleRate / Settings.MinimumFrequency), 1, WindowNumSamples);


		// setup peak picker
		float InverseSensitivity = 1.f - Settings.Sensitivity;
		int32 AutoCorrRange = FMath::Max(PeakPickerMinSize, MaxAutoCorrBin - MinAutoCorrBin);


		FPeakPickerSettings PeakPickerSettings;
		PeakPickerSettings.NumPreMax = FMath::Max(1, PowerScalePeakPickerValue(InverseSensitivity, PeakPickerMinSize, AutoCorrRange / 2));
		PeakPickerSettings.NumPostMax = PeakPickerSettings.NumPreMax;
		PeakPickerSettings.NumPreMean = FMath::Max(1, PowerScalePeakPickerValue(InverseSensitivity, PeakPickerMinSize, AutoCorrRange));
		PeakPickerSettings.NumPostMean = PeakPickerSettings.NumPreMean;
		PeakPickerSettings.NumWait = 1;
		PeakPickerSettings.MeanDelta = 0.f;

		PeakPicker = MakeUnique<FPeakPicker>(PeakPickerSettings);

		// setup block correlator
		FBlockCorrelatorSettings BlockCorrelatorSettings;

		BlockCorrelatorSettings.Log2NumValuesInBlock = Log2WindowNumSamples;
		BlockCorrelatorSettings.WindowType = EWindowType::Blackman;
		BlockCorrelatorSettings.bDoNormalize = true;

		Correlator = MakeUnique<FBlockCorrelator>(BlockCorrelatorSettings);

		if (Correlator.IsValid())
		{
			int32 AutoCorrNumSamples = Correlator->GetNumOutputValues();
			if (AutoCorrNumSamples > 0)
			{
				AutoCorrBuffer.AddUninitialized(AutoCorrNumSamples);
			}
		}
	}

	FAutoCorrelationPitchDetector::~FAutoCorrelationPitchDetector()
	{
	}

	void FAutoCorrelationPitchDetector::DetectPitches(const FAlignedFloatBuffer& InMonoAudio, TArray<FPitchInfo>& OutPitches)
	{
		TAutoSlidingWindow<float, FAudioBufferAlignedAllocator> SlidingWindow(SlidingBuffer, InMonoAudio, WindowBuffer);

		const float InverseSampleRate = 1.f / SampleRate;

		for (FAlignedFloatBuffer& AnalysisBuffer : SlidingWindow)
		{
			int32 SampleCounter = (WindowCounter * SlidingBuffer.GetNumHopSamples()) + SlidingBuffer.GetNumWindowSamples() / 2;
			const float Timestamp = SampleCounter * InverseSampleRate;

			WindowCounter += 1;

			if (!Correlator.IsValid())
			{
				continue;
			}

			Correlator->AutoCorrelate(AnalysisBuffer, AutoCorrBuffer);

			// Pitch peaks
			PeakIndices.Reset();
			int32 MaxBin = 0;

			if (AutoCorrBuffer.Num() > MinAutoCorrBin)
			{
				float Energy = FMath::Max(SMALL_NUMBER, AutoCorrBuffer[0]);

				MaxBin = FMath::Min(AutoCorrBuffer.Num() - MinAutoCorrBin - 1, MaxAutoCorrBin - MinAutoCorrBin);
				if (MaxBin < 1)
				{
					continue;
				}

				TArrayView<float> AutoCorrView(&AutoCorrBuffer.GetData()[MinAutoCorrBin], MaxBin + 1);
				PeakPicker->PickPeaks(AutoCorrView, PeakIndices);

				for (int32 Index : PeakIndices)
				{
					if (Index > MaxBin)
					{
						continue;
					}

					float PeakLocOffset = 0.f;
					float PeakLocMaximum = 0.f;

					const float* AutoCorrViewData = AutoCorrView.GetData();

					// Perform quadratic interpolation on peak.
					// If this returns false, then location is not a true peak.
					if (QuadraticPeakInterpolation(&AutoCorrViewData[Index - 1], PeakLocOffset, PeakLocMaximum))
					{
						// Peak adjusted for offset of AutoCorrView and quadratic interpolation.
						float PeakLoc = static_cast<float>(Index + MinAutoCorrBin) + PeakLocOffset;

						if (PeakLoc < 2.f)
						{
							// Invalid peak location.
							continue;
						}

						FPitchInfo Info;

						Info.Frequency = SampleRate / PeakLoc;
						Info.Strength = FMath::Clamp(PeakLocMaximum / Energy, 0.0f, 1.f);
						Info.Timestamp = Timestamp;

						OutPitches.Add(MoveTemp(Info));
					}
				}
			}
		}
	}

	void FAutoCorrelationPitchDetector::Finalize(TArray<FPitchInfo>& OutPitches)
	{
		SlidingBuffer.Reset();
		WindowCounter = 0;
	}
}
