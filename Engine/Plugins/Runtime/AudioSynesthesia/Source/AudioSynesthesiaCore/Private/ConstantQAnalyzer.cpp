// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstantQAnalyzer.h"

#include "DSP/FloatArrayMath.h"

namespace Audio
{
	FConstantQAnalyzer::FConstantQAnalyzer(const FConstantQAnalyzerSettings& InSettings, const float InSampleRate)
	: Settings(InSettings)
	, SampleRate(InSampleRate)
	, ActualFFTSize(0)
	, NumUsefulFFTBins(0)
    , Window(Settings.WindowType, Settings.FFTSize, 1, false)
	{
		// Need FFTSize atleast 8 to support optimized operations.
		check(Settings.FFTSize >= 8);
		check(SampleRate > 0.f);

		// Create FFT
		FFFTSettings FFTSettings;
		FFTSettings.Log2Size = CeilLog2(Settings.FFTSize);
		FFTSettings.bArrays128BitAligned = true;
		FFTSettings.bEnableHardwareAcceleration = true;

		ActualFFTSize = 1 << FFTSettings.Log2Size;
		NumUsefulFFTBins = (ActualFFTSize / 2) + 1;

		checkf(FFFTFactory::AreFFTSettingsSupported(FFTSettings), TEXT("No fft algorithm supports fft settings."));
		FFT = FFFTFactory::NewFFTAlgorithm(FFTSettings);

		// Create CQT kernel
		CQTTransform = NewPseudoConstantQKernelTransform(Settings, ActualFFTSize, SampleRate);

		if (FFT.IsValid())
		{
			// Size internal buffers
			WindowedSamples.AddZeroed(FFT->NumInputFloats()); // Zero samples to apply zero buffer in case actual FFT is larger than provided settings.

			ComplexSpectrum.AddUninitialized(FFT->NumOutputFloats());
			RealSpectrum.AddUninitialized(ComplexSpectrum.Num() / 2);
		}
	}

	void FConstantQAnalyzer::CalculateCQT(const float* InSamples, TArray<float>& OutCQT)
	{
		// Copy input samples and apply window
		FMemory::Memcpy(WindowedSamples.GetData(), InSamples, sizeof(float) * Settings.FFTSize);
		Window.ApplyToBuffer(WindowedSamples.GetData());

		if (FFT.IsValid())
		{
			FFT->ForwardRealToComplex(WindowedSamples.GetData(), ComplexSpectrum.GetData());

			ArrayComplexToPower(ComplexSpectrum, RealSpectrum);
		
			ScalePowerSpectrumInPlace(ActualFFTSize, FFT->ForwardScaling(), EFFTScaling::None, RealSpectrum);

			if (ESpectrumType::MagnitudeSpectrum == Settings.SpectrumType)
			{
				// Convert to magnitude
				ArraySqrtInPlace(RealSpectrum);
			}
			else
			{
				// Spectrum type should only be magnitude or power. If not true,
				// then there is a possible missing case coverage. 
				check(ESpectrumType::PowerSpectrum == Settings.SpectrumType);
			}

			// Convert spectrum to CQT
			CQTTransform->TransformArray(RealSpectrum, OutCQT);

			// Apply decibel scaling if appropriate.
			if (EConstantQScaling::Decibel == Settings.Scaling)
			{
				// Convert to dB
				switch (Settings.SpectrumType)
				{
					case ESpectrumType::MagnitudeSpectrum:
						ArrayMagnitudeToDecibelInPlace(OutCQT, -90.f);
						break;

					case ESpectrumType::PowerSpectrum:
						ArrayPowerToDecibelInPlace(OutCQT, -90.f);
						break;

					default:
						check(false);
						//checkf(false, TEXT("Unhandled ESpectrumType %s"), GETENUMSTRING(ESpectrumType, Settings.SpectrumType));
						ArrayPowerToDecibelInPlace(OutCQT, -90.f);
				}
			}
		}
	}

	const FConstantQAnalyzerSettings& FConstantQAnalyzer::GetSettings() const
	{
		return Settings;
	}
}

