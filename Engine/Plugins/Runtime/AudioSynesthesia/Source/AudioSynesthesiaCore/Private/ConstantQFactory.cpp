// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstantQFactory.h"
#include "DSP/DeinterleaveView.h"
#include "DSP/SlidingWindow.h"

namespace Audio 
{
	void FConstantQResult::AddFrame(const int32 InChannelIndex, const float InTimestamp, TArrayView<const float> InSpectrum)
	{
		TArray<FConstantQFrame>& FrameArray = ChannelCQTFrames.FindOrAdd(InChannelIndex);
		FrameArray.Emplace(InChannelIndex, InTimestamp, InSpectrum);
	}

	const TArray<FConstantQFrame>& FConstantQResult::GetFramesForChannel(const int32 InChannelIndex) const
	{
		return ChannelCQTFrames[InChannelIndex];
	}

	int32 FConstantQResult::GetNumChannels() const 
	{
		return ChannelCQTFrames.Num();
	}


	FConstantQWorker::FConstantQWorker(const FAnalyzerParameters& InParams, const FConstantQSettings& InAnalyzerSettings)
		: NumChannels(InParams.NumChannels)
		, SampleRate(InParams.SampleRate)
		, NumWindowFrames(InAnalyzerSettings.FFTSize)
		, NumWindowSamples(InAnalyzerSettings.FFTSize * InParams.NumChannels)
		, bDownmixToMono(InAnalyzerSettings.bDownmixToMono)
	{
		check(InParams.NumChannels > 0);
		check(InParams.SampleRate > 0.f);

		ensure(InAnalyzerSettings.AnalysisPeriodInSeconds > 0.f);
		ensure(FMath::IsPowerOfTwo(InAnalyzerSettings.FFTSize));


		NumHopFrames = FMath::Max(1, FMath::RoundToInt(InAnalyzerSettings.AnalysisPeriodInSeconds * InParams.SampleRate));
		NumHopSamples = NumHopFrames * NumChannels;

		if (bDownmixToMono)
		{
			MonoBuffer.Reset(NumWindowFrames);
			MonoBuffer.AddUninitialized(NumWindowFrames);
			// equal power mixing
			MonoScaling = 1.f / FMath::Sqrt(static_cast<float>(NumChannels));
		}

		SlidingBuffer = MakeUnique<TSlidingBuffer<float> >(NumWindowSamples, NumHopSamples);

		ConstantQAnalyzer = MakeUnique<FConstantQAnalyzer>(InAnalyzerSettings, SampleRate);
	}

	void FConstantQWorker::Analyze(TArrayView<const float> InAudio, IAnalyzerResult* OutResult)
	{
		const bool bDoFlush = false;

		AnalyzeMultichannel(InAudio, OutResult, bDoFlush);
	}

	void FConstantQWorker::AnalyzeMultichannel(TArrayView<const float> InAudio, IAnalyzerResult* OutResult, const bool bDoFlush)
	{
		// Assume that outer layers ensured that this is of correct type.
		FConstantQResult* ConstantQResult = static_cast<FConstantQResult*>(OutResult);

		check(nullptr != ConstantQResult);

		TAutoSlidingWindow<float, FAudioBufferAlignedAllocator> SlidingWindow(*SlidingBuffer, InAudio, HopBuffer, bDoFlush);

		if (bDownmixToMono)
		{
			for (const FAlignedFloatBuffer& Window : SlidingWindow)
			{
				FMemory::Memset(MonoBuffer.GetData(), 0, sizeof(float) * NumWindowFrames);

				// Sum all channels into mono buffer
				TAutoDeinterleaveView<float, FAudioBufferAlignedAllocator> DeinterleaveView(Window, ChannelBuffer, NumChannels);
				for(auto Channel : DeinterleaveView)
				{
					ArrayMixIn(Channel.Values, MonoBuffer);		
				}

				// Equal power sum. assuming incoherrent signals.
				ArrayMultiplyByConstantInPlace(MonoBuffer, 1.f / FMath::Sqrt(static_cast<float>(NumChannels)));

				AnalyzeWindow(MonoBuffer, 0, *ConstantQResult);

				NumBuffers++;
			}
		}
		else
		{
			// Loop through entire array of input samples.
			for (const FAlignedFloatBuffer& Window : SlidingWindow)
			{
				// Each channel is analyzed seperately.
				for (const TDeinterleaveView<float>::TChannel<FAudioBufferAlignedAllocator> Channel : TAutoDeinterleaveView<float, FAudioBufferAlignedAllocator>(Window, ChannelBuffer, NumChannels))
				{
					AnalyzeWindow(Channel.Values, Channel.ChannelIndex, *ConstantQResult);
				}

				NumBuffers++;
			}
		}
	}

	void FConstantQWorker::AnalyzeWindow(const FAlignedFloatBuffer& InWindow, const int32 InChannelIndex, FConstantQResult& OutResult)
	{
		// Run CQT Analyzer
		ConstantQAnalyzer->CalculateCQT(InWindow.GetData(), CQTSpectrum);

		// Get timestamp as center of audio window.
		const float Timestamp = ((NumBuffers * NumHopFrames) + (0.5f * ConstantQAnalyzer->GetSettings().FFTSize)) / SampleRate;

		OutResult.AddFrame(InChannelIndex, Timestamp, CQTSpectrum);
	}

	// Name of this analyzer type.
	FName FConstantQFactory::GetName() const
	{
		static FName FactoryName(TEXT("ConstantQFactory"));
		return FactoryName;
	}

	// Human readable name of this analyzer.
	FString FConstantQFactory::GetTitle() const
	{
		return TEXT("Real-Time Constant Q Analyzer");
	}

	// Create a new FConstantQResult.
	TUniquePtr<IAnalyzerResult> FConstantQFactory::NewResult() const
	{
		return MakeUnique<FConstantQResult>();
	}

	// Create a new FConstantQWorker
	TUniquePtr<IAnalyzerWorker> FConstantQFactory::NewWorker(const FAnalyzerParameters& InParams, const IAnalyzerSettings* InSettings) const
	{
		const FConstantQSettings* ConstantQSettings = static_cast<const FConstantQSettings*>(InSettings);

		check(nullptr != ConstantQSettings);

		return MakeUnique<FConstantQWorker>(InParams, *ConstantQSettings);
	}
}
