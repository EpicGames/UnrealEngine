// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstantQNRTFactory.h"
#include "AudioSynesthesiaCustomVersion.h"
#include "DSP/DeinterleaveView.h"
#include "DSP/FloatArrayMath.h"
#include "DSP/SlidingWindow.h"

namespace Audio 
{
	/*******************************************************************************/
	/*************************** FConstantQNRTResult *******************************/
	/*******************************************************************************/

	FArchive &operator <<(FArchive& Ar, FConstantQFrame& Frame)
	{
		Ar.UsingCustomVersion(FAudioSynesthesiaCustomVersion::GUID);

		Ar << Frame.Channel;
		Ar << Frame.Timestamp;
		Ar << Frame.Spectrum;

		return Ar;
	}

	FConstantQNRTResult::FConstantQNRTResult()
	:	DurationInSeconds(0.f)
	,	bIsSortedChronologically(false)
	{}

	void FConstantQNRTResult::Serialize(FArchive& Archive)
	{
		Archive.UsingCustomVersion(FAudioSynesthesiaCustomVersion::GUID);
		
		Archive << DurationInSeconds;
		Archive << bIsSortedChronologically;
		Archive << ChannelCQTFrames;
		Archive << ChannelCQTIntervals;
	}

	void FConstantQNRTResult::AddFrame(int32 InChannelIndex, float InTimestamp, TArrayView<const float> InSpectrum)
	{
		TArray<FConstantQFrame>& FrameArray = ChannelCQTFrames.FindOrAdd(InChannelIndex);
		FrameArray.Emplace(InChannelIndex, InTimestamp, InSpectrum);

		FFloatInterval& Interval = ChannelCQTIntervals.FindOrAdd(InChannelIndex);
		for (float Value : InSpectrum)
		{
			Interval.Include(Value);
		}

		bIsSortedChronologically = false;
	}

	bool FConstantQNRTResult::ContainsChannel(int32 InChannelIndex) const
	{
		return ChannelCQTFrames.Contains(InChannelIndex);
	}

	const TArray<FConstantQFrame>& FConstantQNRTResult::GetFramesForChannel(int32 InChannelIndex) const
	{
		return ChannelCQTFrames[InChannelIndex];
	}

	FFloatInterval FConstantQNRTResult::GetChannelConstantQInterval(int32 InChannelIdx) const
	{
		return ChannelCQTIntervals[InChannelIdx];
	}

	void FConstantQNRTResult::GetChannels(TArray<int32>& OutChannels) const
	{
		ChannelCQTFrames.GetKeys(OutChannels);
	}

	float FConstantQNRTResult::GetDurationInSeconds() const 
	{
		return DurationInSeconds;
	}

	void FConstantQNRTResult::SetDurationInSeconds(float InDuration)
	{
		DurationInSeconds = InDuration;
	}

	 /* Returns true if FConstantQNRTFrame arrays are sorted in chronologically ascending order via their timestamp.
	 */
	bool FConstantQNRTResult::IsSortedChronologically() const
	{
		return bIsSortedChronologically;
	}

	/**
	 * Sorts FConstantQNRTFrame arrays in chronologically ascnding order via their timestamp.
	 */
	void FConstantQNRTResult::SortChronologically()
	{
		for (auto& KeyValue : ChannelCQTFrames)
		{
			KeyValue.Value.Sort([](const FConstantQFrame& A, const FConstantQFrame& B) { return A.Timestamp < B.Timestamp; });
		}

		bIsSortedChronologically = true;
	}

	/*******************************************************************************/
	/*************************** FConstantQNRTWorker *******************************/
	/*******************************************************************************/

	FConstantQNRTWorker::FConstantQNRTWorker(const FAnalyzerNRTParameters& InParams, const FConstantQNRTSettings& InAnalyzerSettings)
	:	NumFrames(0)
	,	NumChannels(InParams.NumChannels)
	,	NumBuffers(0)
	,	SampleRate(InParams.SampleRate)
	,	NumHopFrames(0)
	,	NumHopSamples(0)
	,	NumWindowFrames(InAnalyzerSettings.FFTSize)
	,	NumWindowSamples(InAnalyzerSettings.FFTSize * InParams.NumChannels)
	,	MonoScaling(1.f)
	,	bDownmixToMono(InAnalyzerSettings.bDownmixToMono)
	{
		check(InParams.NumChannels > 0);
		check(InParams.SampleRate > 0.f);

		check(InAnalyzerSettings.AnalysisPeriod > 0.f);
		check(FMath::IsPowerOfTwo(InAnalyzerSettings.FFTSize));


		NumHopFrames = FMath::Max(1, FMath::RoundToInt(InAnalyzerSettings.AnalysisPeriod * InParams.SampleRate));
		NumHopSamples = NumHopFrames * NumChannels;

		if (bDownmixToMono)
		{
			MonoBuffer.Reset(NumWindowFrames);
			MonoBuffer.AddUninitialized(NumWindowFrames);
			// equal power mixing
			MonoScaling = 1.f / FMath::Sqrt(static_cast<float>(NumChannels));
		}

		SlidingBuffer = MakeUnique<TSlidingBuffer<float> >(NumWindowSamples, NumHopSamples);

		ConstantQAnalyzer = MakeUnique<FConstantQAnalyzer>(InAnalyzerSettings, SampleRate);
	}


	void FConstantQNRTWorker::Analyze(TArrayView<const float> InAudio, IAnalyzerNRTResult* OutResult)
	{
		bool bDoFlush = false;

		AnalyzeMultichannel(InAudio, OutResult, bDoFlush);
	}

	void FConstantQNRTWorker::AnalyzeMultichannel(TArrayView<const float> InAudio, IAnalyzerNRTResult* OutResult, bool bDoFlush)
	{
		NumFrames += InAudio.Num() / NumChannels;

		// Assume that outer layers ensured that this is of correct type.
		FConstantQNRTResult* ConstantQResult = static_cast<FConstantQNRTResult*>(OutResult);

		check(nullptr != ConstantQResult);

		TAutoSlidingWindow<float, FAudioBufferAlignedAllocator> SlidingWindow(*SlidingBuffer, InAudio, HopBuffer, bDoFlush);

		if (bDownmixToMono)
		{
			for (const FAlignedFloatBuffer& Window : SlidingWindow)
			{
				FMemory::Memset(MonoBuffer.GetData(), 0, sizeof(float) * NumWindowFrames);

				// Sum all channels into mono buffer
				TAutoDeinterleaveView<float, FAudioBufferAlignedAllocator> DeinterleaveView(Window, ChannelBuffer, NumChannels);
				for(auto Channel : DeinterleaveView)
				{
					ArrayMixIn(Channel.Values, MonoBuffer);		
				}

				// Equal power sum. assuming incoherrent signals.
				ArrayMultiplyByConstantInPlace(MonoBuffer, 1.f / FMath::Sqrt(static_cast<float>(NumChannels)));

				AnalyzeWindow(MonoBuffer, 0, *ConstantQResult);

				NumBuffers++;
			}
		}
		else
		{
			// Loop through entire array of input samples.
			for (const FAlignedFloatBuffer& Window : SlidingWindow)
			{
				// Each channel is analyzed seperately.
				for(const TDeinterleaveView<float>::TChannel<FAudioBufferAlignedAllocator> Channel : TAutoDeinterleaveView<float, FAudioBufferAlignedAllocator>(Window, ChannelBuffer, NumChannels))
				{
					AnalyzeWindow(Channel.Values, Channel.ChannelIndex, *ConstantQResult);
				}

				NumBuffers++;
			}
		}
	}

	/** Called when analysis of audio asset is complete. */
	void FConstantQNRTWorker::Finalize(IAnalyzerNRTResult* OutResult) 
	{
		FAlignedFloatBuffer EmptyArray;
		bool bDoFlush = true;

		AnalyzeMultichannel(EmptyArray, OutResult, bDoFlush);

		// Assume that outer layers ensured that this is of correct type.
		FConstantQNRTResult* ConstantQResult = static_cast<FConstantQNRTResult*>(OutResult);

		check(nullptr != ConstantQResult);


		if (nullptr != OutResult)
		{
			check(SampleRate > 0.f);

			float DurationInSeconds = static_cast<float>(NumFrames) / SampleRate;

			ConstantQResult->SetDurationInSeconds(DurationInSeconds);

			ConstantQResult->SortChronologically();
		}

		// Reset internal counters
		NumBuffers = 0; 
		NumFrames = 0;

		SlidingBuffer.Reset();
	}

	void FConstantQNRTWorker::AnalyzeWindow(const FAlignedFloatBuffer& InWindow, int32 InChannelIndex, FConstantQNRTResult& OutResult)
	{
		// Run CQT Analyzer
		ConstantQAnalyzer->CalculateCQT(InWindow.GetData(), CQTSpectrum);

		// Get timestamp as center of audio window.
		float Timestamp = ((NumBuffers * NumHopFrames) + (0.5f * ConstantQAnalyzer->GetSettings().FFTSize)) / SampleRate;

		OutResult.AddFrame(InChannelIndex, Timestamp, CQTSpectrum);
	}

	// Name of this analyzer type.
	FName FConstantQNRTFactory::GetName() const
	{
		static FName FactoryName(TEXT("ConstantQNRTFactory"));
		return FactoryName;
	}

	// Human readable name of this analyzer.
	FString FConstantQNRTFactory::GetTitle() const
	{
		return TEXT("Constant Q Analyzer Non-Real-Time");
	}

	// Create a new FConstantQNRTResult.
	TUniquePtr<IAnalyzerNRTResult> FConstantQNRTFactory::NewResult() const
	{
		return MakeUnique<FConstantQNRTResult>();
	}

	// Create a new FConstantQNRTWorker
	TUniquePtr<IAnalyzerNRTWorker> FConstantQNRTFactory::NewWorker(const FAnalyzerNRTParameters& InParams, const IAnalyzerNRTSettings* InSettings) const
	{
		const FConstantQNRTSettings* ConstantQSettings = static_cast<const FConstantQNRTSettings*>(InSettings);

		check(nullptr != ConstantQSettings);

		return MakeUnique<FConstantQNRTWorker>(InParams, *ConstantQSettings);
	}
}

