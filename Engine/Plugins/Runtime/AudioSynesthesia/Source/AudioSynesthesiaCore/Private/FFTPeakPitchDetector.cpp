// Copyright Epic Games, Inc. All Rights Reserved.

#include "FFTPeakPitchDetector.h"

#include "AudioSynesthesiaCoreLog.h"
#include "PeakPicker.h"
#include "DSP/Dsp.h"
#include "DSP/FFTAlgorithm.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	namespace FFTPeakPitchDetectorIntrinsics
	{
		constexpr int32 PeakPickerMinSize = 2;

		int32 PowerScalePeakPickerValue(float InValue, int32 MinimumValue, int32 MaximumValue)
		{
			MinimumValue = FMath::Max(2, MinimumValue);
			MaximumValue = FMath::Max(2, MaximumValue);
			InValue = FMath::Clamp(InValue, 0.f, 1.f);

			float Power = FMath::Max(SMALL_NUMBER, FMath::Loge(static_cast<float>(MaximumValue)) / FMath::Loge(static_cast<float>(MinimumValue)));
			float Denom = FMath::Max(SMALL_NUMBER, Power - 1.f);

			// Power mapped scale between 0 and 1.
			const float Scale = (FMath::Pow(Power, InValue) - 1.f) / Denom;

			float Range = static_cast<float>(MaximumValue - MinimumValue);

			return FMath::Clamp(MinimumValue + FMath::RoundToInt(Range * Scale), MinimumValue, MaximumValue);
		}
	}

	FFFTPeakPitchDetector::FFFTPeakPitchDetector(const FFFTPeakPitchDetectorSettings& InSettings, float InSampleRate)
	:	Settings(InSettings)
	,	SampleRate(InSampleRate)
	,	FFTSize(1 << InSettings.Log2FFTSize)
	,	MinFFTBin(1)
	,	MaxFFTBin(1)
	,	FFTScaling(1.f)
	,	WindowCounter(0)
	,	SlidingBuffer(1, 1)
	{
		using namespace FFTPeakPitchDetectorIntrinsics;

		if (!ensure(SampleRate > 0.f))
		{
			SampleRate = 1.f;
		}

		int32 AnalysisWindowNumSamples = FMath::RoundToInt(Settings.AnalysisWindowSeconds * SampleRate);
		int32 AnalysisHopNumSamples = FMath::RoundToInt(Settings.AnalysisHopSeconds * SampleRate);

		if (AnalysisWindowNumSamples > FFTSize)
		{
			UE_LOGF(LogAudioSynesthesiaCore, Log, "The analysis window size [%d] is being reduced to the FFT size [%d]", AnalysisWindowNumSamples, FFTSize);
			AnalysisWindowNumSamples = FFTSize;
		}

		UE_CLOGF(AnalysisHopNumSamples > AnalysisWindowNumSamples, LogAudioSynesthesiaCore, Log, "Some samples will not be anlayzed because the hop size [%d] is larger than the analysis window [%d]", AnalysisHopNumSamples, AnalysisWindowNumSamples);

		SlidingBuffer = TSlidingBuffer<float>(AnalysisWindowNumSamples, AnalysisHopNumSamples);

		// setup fft
		FFFTSettings FFTSettings;
		FFTSettings.Log2Size = Settings.Log2FFTSize;
		FFTSettings.bArrays128BitAligned = true;
		FFTSettings.bEnableHardwareAcceleration = true;

		FFTAlgorithm = FFFTFactory::NewFFTAlgorithm(FFTSettings);
		check(FFTAlgorithm.IsValid());

		switch (FFTAlgorithm->ForwardScaling())
		{

			case EFFTScaling::MultipliedByFFTSize:
				FFTScaling = 1.f / FFTAlgorithm->Size();
				break;

			case EFFTScaling::MultipliedBySqrtFFTSize:
				FFTScaling = 1.f / FMath::Sqrt(static_cast<float>(FFTAlgorithm->Size()));
				break;

			case EFFTScaling::DividedByFFTSize:
				FFTScaling = FFTAlgorithm->Size();
				break;

			case EFFTScaling::DividedBySqrtFFTSize:
				FFTScaling = FMath::Sqrt(static_cast<float>(FFTAlgorithm->Size()));
				break;

			case EFFTScaling::None:
			default:
				FFTScaling = 1.f;
				break;
		}

		float HzPerBin = SampleRate / FFTSize;
		MinFFTBin = FMath::Max(1, FMath::FloorToInt(Settings.MinimumFrequency / HzPerBin));
		MaxFFTBin = FMath::Max(1, FMath::CeilToInt(Settings.MaximumFrequency / HzPerBin));
		int32 FFTRange = MaxFFTBin - MinFFTBin + 1;

		// setup peak picker
		float InverseSensitivity = 1.f - Settings.Sensitivity;
		
		FPeakPickerSettings PeakPickerSettings;
		PeakPickerSettings.NumPreMax = FMath::Max(1, PowerScalePeakPickerValue(InverseSensitivity, PeakPickerMinSize, FFTRange / 2));
		PeakPickerSettings.NumPostMax = PeakPickerSettings.NumPreMax;
		PeakPickerSettings.NumPreMean = FMath::Max(1, PowerScalePeakPickerValue(InverseSensitivity, PeakPickerMinSize, FFTRange));

		PeakPickerSettings.NumPostMean = PeakPickerSettings.NumPreMean;
		PeakPickerSettings.NumWait = 1;
		PeakPickerSettings.MeanDelta = 0.f;

		PeakPicker = MakeUnique<FPeakPicker>(PeakPickerSettings);

		// setup some extra buffers
		if (FFTAlgorithm->NumInputFloats() > 0)
		{
			ZeroPaddedAnalysisBuffer.AddZeroed(FFTAlgorithm->NumInputFloats());
		}

		if (FFTAlgorithm->NumOutputFloats() > 0)
		{
			ComplexSpectrumBuffer.AddZeroed(FFTAlgorithm->NumOutputFloats());

			PowerSpectrumBuffer.AddZeroed(FMath::Max(1, FFTAlgorithm->NumOutputFloats() / 2));
		}
	}

	FFFTPeakPitchDetector::~FFFTPeakPitchDetector()
	{
	}

	void FFFTPeakPitchDetector::DetectPitches(const FAlignedFloatBuffer& InMonoAudio, TArray<FPitchInfo>& OutPitches)
	{
		TAutoSlidingWindow<float, FAudioBufferAlignedAllocator> SlidingWindow(SlidingBuffer, InMonoAudio, WindowBuffer);

		const float InverseMultiply = 1.f / SampleRate;

		for (FAlignedFloatBuffer& AnalysisBuffer : SlidingWindow)
		{
			int32 SampleCounter = (WindowCounter * SlidingBuffer.GetNumHopSamples()) + SlidingBuffer.GetNumWindowSamples() / 2;
			float Timestamp = SampleCounter * InverseMultiply;

			WindowCounter += 1;

			// FFT
			FMemory::Memcpy(ZeroPaddedAnalysisBuffer.GetData(), AnalysisBuffer.GetData(), AnalysisBuffer.Num() * sizeof(float));
			FFTAlgorithm->ForwardRealToComplex(ZeroPaddedAnalysisBuffer.GetData(), ComplexSpectrumBuffer.GetData());
			ArrayMultiplyByConstantInPlace(ComplexSpectrumBuffer, FFTScaling);

			ArrayComplexToPower(ComplexSpectrumBuffer, PowerSpectrumBuffer);

			// Pitch peaks
			TArray<int32> PeakIndices;
			PeakPicker->PickPeaks(PowerSpectrumBuffer, PeakIndices);

			PeakIndices = PeakIndices.FilterByPredicate([&](int32 Index){
				return (Index >= MinFFTBin) && (Index <= MaxFFTBin);
			});


			for (int32 Index : PeakIndices)
			{
				float PeakLocOffset = 0.f;
				float PeakLocMaximum = 0.f;

				const float* PowerSpectrumData = PowerSpectrumBuffer.GetData();

				// Perform quadratic interpolation on peak.
				// If this returns false, then location is not a true peak.
				if (QuadraticPeakInterpolation(&PowerSpectrumData[Index - 1], PeakLocOffset, PeakLocMaximum))
				{
					FPitchInfo Info;

					Info.Frequency = (static_cast<float>(Index) + PeakLocOffset) * SampleRate / static_cast<float>(FFTAlgorithm->Size());
					Info.Strength = PeakLocMaximum;
					Info.Timestamp = Timestamp;

					OutPitches.Add(MoveTemp(Info));
				}
			}

			// Get median of work buffer
			if (PowerSpectrumBuffer.Num() > 0)
			{
				// The strength adjustment is applied after the locations are determined because it requires us
				// to sort the 
				PowerSpectrumBuffer.Sort();

				float MedianValue = FMath::Max(SMALL_NUMBER, PowerSpectrumBuffer[PowerSpectrumBuffer.Num() / 2]);

				for (FPitchInfo& Info : OutPitches)
				{
					float DecibelAboveMedian = 10.f * FMath::LogX(10.f, FMath::Max(Info.Strength / MedianValue, SMALL_NUMBER));
					
					Info.Strength = FMath::Clamp(DecibelAboveMedian / Settings.MaxStrengthSNRDecibels, 0.0f, 1.f);
				}

			}

		}
	}

	void FFFTPeakPitchDetector::Finalize(TArray<FPitchInfo>& OutPitches)
	{
		SlidingBuffer.Reset();
		WindowCounter = 0;
	}
}
