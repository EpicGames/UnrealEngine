// Copyright Epic Games, Inc. All Rights Reserved.

#include "LKFSFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LKFSFactory)

namespace Audio
{
	namespace LKFSFactoryPrivate
	{
		FLKFSAnalyzer MakeLKFSAnalyzer(const FAnalyzerParameters& InParams, const FLKFSSettings& InAnalyzerSettings)
		{
			const FLKFSAnalyzerSettings LKFSAnalyzerSettings
			{
				.LoudnessAnalyzerSettings          = InAnalyzerSettings.LoudnessAnalyzerSettings,
				.AnalysisPeriod                    = InAnalyzerSettings.AnalysisPeriod,
				.AnalysisWindowDuration            = InAnalyzerSettings.AnalysisWindowDuration,
				.ShortTermLoudnessDuration         = InAnalyzerSettings.ShortTermLoudnessDuration,
				.IntegratedLoudnessAnalysisPeriod  = InAnalyzerSettings.IntegratedLoudnessAnalysisPeriod,
				.IntegratedLoudnessDuration        = InAnalyzerSettings.IntegratedLoudnessDuration,
				.bCalculatePerChannelLoudnessRange = InAnalyzerSettings.bCalculatePerChannelLoudnessRange,
				.bCalculateOverallLoudnessRange    = InAnalyzerSettings.bCalculateOverallLoudnessRange
			};

			return FLKFSAnalyzer(InParams.NumChannels, static_cast<float>(InParams.SampleRate), LKFSAnalyzerSettings);
		}
	} // namespace LKFSFactoryPrivate


	//----------------------------------------------------------------------------------------
	// FLKFSAnalyzerResult
	//----------------------------------------------------------------------------------------
	int32 FLKFSAnalyzerResult::GetNumChannels() const
	{
		int32 Num = ChannelResults.Num();
		if (Num > 0)
		{
			return Num - 1; // Subtract one for storage of "Overall" channel
		}
		return 0;
	}

	void FLKFSAnalyzerResult::Add(FLKFSResults InDatum)
	{
		TArray<FLKFSResults>& ChannelData = ChannelResults.FindOrAdd(InDatum.Channel);
		ChannelData.Add(MoveTemp(InDatum));
	}

	const TArray<FLKFSResults>& FLKFSAnalyzerResult::GetChannelLoudnessResults(int32 ChannelIdx) const
	{
		if (const TArray<FLKFSResults>* ChannelData = ChannelResults.Find(ChannelIdx))
		{
			return *ChannelData;
		}
		else
		{
			static const TArray<FLKFSResults> EmptyArray;
			return EmptyArray;
		}
	}

	const TArray<FLKFSResults>& FLKFSAnalyzerResult::GetLoudnessResults() const
	{
		return GetChannelLoudnessResults(ChannelIndexOverall);
	}


	//----------------------------------------------------------------------------------------
	// FLKFSWorker
	//----------------------------------------------------------------------------------------
	FLKFSWorker::FLKFSWorker(const FAnalyzerParameters& InParams, const FLKFSSettings& InAnalyzerSettings)
		: NumChannels(InParams.NumChannels)
		, LKFSAnalyzer(LKFSFactoryPrivate::MakeLKFSAnalyzer(InParams, InAnalyzerSettings))
	{
	}

	void FLKFSWorker::Analyze(TArrayView<const float> InAudio, IAnalyzerResult* OutResult) 
	{
		// Assume that outer layers ensured that this is of correct type.
		FLKFSAnalyzerResult* LoudnessResult = static_cast<FLKFSAnalyzerResult*>(OutResult);
		check(LoudnessResult != nullptr);
		
		FLKFSAnalyzerResults LKFSAnalyzerResults;
		LKFSAnalyzer.Analyze(InAudio, LKFSAnalyzerResults);

		// Per Channel results
		for (int32 Index = 0; Index < NumChannels; ++Index)
		{
			for (const FLKFSResult& ChannelLoudnessResult : LKFSAnalyzerResults.GetChannelLoudnessResults(Index))
			{
				LoudnessResult->Add(
					{
						.Channel            = ChannelLoudnessResult.Channel,
						.Timestamp          = ChannelLoudnessResult.Timestamp,
						.Energy             = ChannelLoudnessResult.Energy,
						.Loudness           = ChannelLoudnessResult.Loudness,
						.ShortTermLoudness  = ChannelLoudnessResult.ShortTermLoudness,
						.IntegratedLoudness = ChannelLoudnessResult.IntegratedLoudness,
						.GatedLoudness      = ChannelLoudnessResult.GatedLoudness,
						.LoudnessRange      = ChannelLoudnessResult.LoudnessRange
					}
				);
			}
		}

		// Overall results
		for (const FLKFSResult& OverallLoudnessResult : LKFSAnalyzerResults.GetLoudnessResults())
		{
			LoudnessResult->Add(
				{
					.Channel            = OverallLoudnessResult.Channel,
					.Timestamp          = OverallLoudnessResult.Timestamp,
					.Energy             = OverallLoudnessResult.Energy,
					.Loudness           = OverallLoudnessResult.Loudness,
					.ShortTermLoudness  = OverallLoudnessResult.ShortTermLoudness,
					.IntegratedLoudness = OverallLoudnessResult.IntegratedLoudness,
					.GatedLoudness      = OverallLoudnessResult.GatedLoudness,
					.LoudnessRange      = OverallLoudnessResult.LoudnessRange
				}
			);
		}
	}

	FName FLKFSFactory::GetName() const 
	{
		static FName FactoryName(TEXT("LKFSFactory"));
		return FactoryName;
	}

	FString FLKFSFactory::GetTitle() const
	{
		return TEXT("LKFS Analyzer Real-Time");
	}

	TUniquePtr<IAnalyzerResult> FLKFSFactory::NewResult() const
	{
		TUniquePtr<FLKFSAnalyzerResult> Result = MakeUnique<FLKFSAnalyzerResult>();
		return Result;
	}

	TUniquePtr<IAnalyzerWorker> FLKFSFactory::NewWorker(const FAnalyzerParameters& InParams, const IAnalyzerSettings* InSettings) const
	{
		const FLKFSSettings* LoudnessSettings = static_cast<const FLKFSSettings*>(InSettings);

		check(nullptr != LoudnessSettings);

		return MakeUnique<FLKFSWorker>(InParams, *LoudnessSettings);
	}
} // namespace Audio
