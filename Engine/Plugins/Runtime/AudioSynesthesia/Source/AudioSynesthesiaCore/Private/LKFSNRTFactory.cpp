// Copyright Epic Games, Inc. All Rights Reserved.

#include "LKFSNRTFactory.h"

#include "Algo/IsSorted.h"
#include "AudioSynesthesiaCoreLog.h"
#include "DSP/LKFSUtils.h"
#include "DSP/SlidingWindow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LKFSNRTFactory)

namespace Audio
{
	namespace LKFSNRTFactoryPrivate
	{
		float CalculateGatedLoudnessPass(TArrayView<const FLKFSNRTResults> InLoudnessData, float InLoudnessThreshold)
		{
			// Get average loudness of values about gating threshold
			double AccumEnergy = 0.f;
			int32 AccumNum = 0;

			for (const FLKFSNRTResults& Datum : InLoudnessData)
			{
				if (Datum.Loudness > InLoudnessThreshold)
				{
					AccumEnergy += Datum.Energy;
					AccumNum++;
				}
			}

			if (AccumNum > 0)
			{
				float AverageEnergy = (float)(AccumEnergy / (double)AccumNum);
				return FLoudnessAnalyzer::ConvertPerceptualEnergyToLKFS(AverageEnergy);
			}
			else
			{
				// No values exceeded threshold. Using the threshold seems like a
				// reasonable result.
				return InLoudnessThreshold;
			}
		}

		float CalculateGatedLoudness(TArrayView<const FLKFSNRTResults> InLoudnessData)
		{
			if (InLoudnessData.Num() < 1)
			{
				return FLKFSNRTAggregateStats::InvalidLoudness;;
			}
			// Gated Loudness only examines loudnesses above a certain gating loudness. 
			//
			// It is a 2 pass algorithm 
			// Pass 1: Use a set loudness threshold of -70lkfs to determine an initial gated loudness
			// Pass 2: Use the initial gated loudness to derive a new loudness threshold and calculate the gate loudness again. 
			constexpr float InitLoudnessThreshold = -70.f;

			const float GatedLoudnessPass1 = CalculateGatedLoudnessPass(InLoudnessData, InitLoudnessThreshold);
			const float NewThreshold = GatedLoudnessPass1 - 10.f;
			return CalculateGatedLoudnessPass(InLoudnessData, NewThreshold);
		}
		
		void CalculateShortTermAndIntegratedLoudness(float InShortTermLoudnessDuration, TArrayView<FLKFSNRTResults> InOutSortedLoudnessData, FLKFSNRTAggregateStats& InOutStats)
		{
			check(InShortTermLoudnessDuration > 0.f);
			checkSlow(Algo::IsSortedBy(InOutSortedLoudnessData, [](const FLKFSNRTResults& InDatum) { return InDatum.Timestamp; }));

			if (InOutSortedLoudnessData.Num() < 1)
			{
				InOutStats.IntegratedLoudness = FLKFSNRTAggregateStats::InvalidLoudness;
				return;
			}

			// Track a short term window. To find the energy inside of the short term window,
			// accumulate the energy over all the data. For a given short term window, the 
			// energy in that window can be determined by subtracting the tail accumulated
			// energy from the head accumulated energy. 
			

			double HeadAccumEnergy = 0; // Total accumulated energy at head
			double ShortTermTailAccumEnergy = 0; // Total accumulated energy at tail
			const float FirstTimestamp = InOutSortedLoudnessData[0].Timestamp;
			double ShortTermLoudnessNum = 0;
			
			const FLKFSNRTResults* ShortTermLoudnessDatumBegin = &InOutSortedLoudnessData[0];

			for (int32 Index = 0; Index < InOutSortedLoudnessData.Num(); Index++)
			{
				FLKFSNRTResults& LoudnessDatum = InOutSortedLoudnessData[Index];
				// Acummulated loudness at head
				HeadAccumEnergy += LoudnessDatum.Energy;
				
				// Only accumulated loudness at tail if we are within the desired window
				float MinTimestamp = LoudnessDatum.Timestamp - InShortTermLoudnessDuration;
				if (MinTimestamp < FirstTimestamp)
				{
					// Grow the number of elements in the window since the difference in timestamps
					// between the head and tail of the data are less than the window size
					ShortTermLoudnessNum = (float)(Index + 1);
				}
				else
				{
					ShortTermTailAccumEnergy += ShortTermLoudnessDatumBegin->Energy;
					ShortTermLoudnessDatumBegin++;
				}

				checkf((LoudnessDatum.Timestamp - ShortTermLoudnessDatumBegin->Timestamp) <= (InShortTermLoudnessDuration + UE_KINDA_SMALL_NUMBER), TEXT("Invalid short term loudness window. The algorithm for integrated and short term loudness assumes a data to be regularly spaced in time"));

				// Short term loudness is the accumulated loudness divided the number of elements in the window.
				float ShortTermEnergy = (float)((HeadAccumEnergy - ShortTermTailAccumEnergy) / ShortTermLoudnessNum);
				LoudnessDatum.ShortTermLoudness = FLoudnessAnalyzer::ConvertPerceptualEnergyToLKFS(ShortTermEnergy);
			}

			// Integrated loudness is the average over the entire file.
			float IntegratedEnergy = (float)(HeadAccumEnergy / (double)InOutSortedLoudnessData.Num());
			InOutStats.IntegratedLoudness = FLoudnessAnalyzer::ConvertPerceptualEnergyToLKFS(IntegratedEnergy);
		}


		// Calculates ShortTermLoudness (part of FLKFSNRTResults) and FLKFSNRTAggregateStats
		void CalculateAggregateLoudnessStats(float InShortTermLoudnessDuration, TArrayView<FLKFSNRTResults> InOutSortedLoudnessData, FLKFSNRTAggregateStats& InOutStats)
		{
			CalculateShortTermAndIntegratedLoudness(InShortTermLoudnessDuration, InOutSortedLoudnessData, InOutStats);
			InOutStats.GatedLoudness = CalculateGatedLoudness(InOutSortedLoudnessData);
		}
	}
}

namespace Audio
{
	/************************************************************************/
	/********************** FLKFSNRTResult ******************************/
	/************************************************************************/

	// Use for overall loudness storage
	const int32 FLKFSNRTResult::ChannelIndexOverall = -1;

	FLKFSNRTResult::FLKFSNRTResult()
	{
	}

	void FLKFSNRTResult::AddChannel(int32 InChannelIndex)
	{
		if (!Data.ChannelData.Contains(InChannelIndex))
		{
			Data.ChannelData.Add(InChannelIndex);
		}
	}

	void FLKFSNRTResult::Add(const FLKFSNRTResults& InDatum)
	{
		FLKFSNRTChannelData& ChannelDatum = Data.ChannelData.FindOrAdd(InDatum.Channel);
		// Store loudness data in appropriate channel
		ChannelDatum.LoudnessArray.Add(InDatum);

		// Mark as not sorted
		Data.bIsSortedChronologically = false;
	}

	bool FLKFSNRTResult::ContainsChannel(int32 InChannelIndex) const
	{
		return Data.ChannelData.Contains(InChannelIndex);
	}

	TArrayView<const FLKFSNRTResults> FLKFSNRTResult::GetChannelLoudnessArray(int32 ChannelIdx) const
	{
		if (const FLKFSNRTChannelData* ChannelDatum = Data.ChannelData.Find(ChannelIdx))
		{
			return ChannelDatum->LoudnessArray;
		}
		return {};
	}

	void FLKFSNRTResult::Serialize(FArchive& Archive)
	{
		FLKFSNRTWaveData::StaticStruct()->SerializeItem(Archive, &Data, nullptr);
	}

	TArrayView<FLKFSNRTResults> FLKFSNRTResult::GetMutableChannelLoudnessArray(int32 ChannelIdx)
	{
		if (FLKFSNRTChannelData* ChannelDatum = Data.ChannelData.Find(ChannelIdx))
		{
			return ChannelDatum->LoudnessArray;
		}
		return {};
	}

	TArrayView<const FLKFSNRTResults> FLKFSNRTResult::GetLoudnessArray() const
	{
		return GetChannelLoudnessArray(ChannelIndexOverall);
	}

	const FLKFSNRTAggregateStats* FLKFSNRTResult::FindAggregateLoudnessStats(int32 InChannelIdx) const
	{
		if (const FLKFSNRTChannelData* ChannelDatum = Data.ChannelData.Find(InChannelIdx))
		{
			return &ChannelDatum->AggregateStats;
		}
		return nullptr;
	}

	void FLKFSNRTResult::SetAggregateLoudnessStats(int32 InChannelIdx, FLKFSNRTAggregateStats InStats)
	{
		Data.ChannelData.FindOrAdd(InChannelIdx).AggregateStats = InStats;
	}

	void FLKFSNRTResult::GetChannels(TArray<int32>& OutChannels) const
	{
		Data.ChannelData.GetKeys(OutChannels);
	}

	float FLKFSNRTResult::GetDurationInSeconds() const 
	{
		return Data.DurationInSeconds;
	}

	void FLKFSNRTResult::SetDurationInSeconds(float InDuration)
	{
		Data.DurationInSeconds = InDuration;
	}

	bool FLKFSNRTResult::IsSortedChronologically() const
	{
		return Data.bIsSortedChronologically;
	}

	void FLKFSNRTResult::SortChronologically()
	{
		for (auto& KeyValue : Data.ChannelData)
		{
			KeyValue.Value.LoudnessArray.Sort([](const FLKFSNRTResults& A, const FLKFSNRTResults& B) { return A.Timestamp < B.Timestamp; });
		}

		Data.bIsSortedChronologically = true;
	}

	/************************************************************************/
	/********************** FLKFSNRTWorker ******************************/
	/************************************************************************/

	FLKFSNRTWorker::FLKFSNRTWorker(const FAnalyzerNRTParameters& InParams, const FLKFSNRTSettings& InAnalyzerSettings)
	: NumChannels(InParams.NumChannels)
	, SampleRate(InParams.SampleRate)
	, ShortTermLoudnessDuration(InAnalyzerSettings.ShortTermLoudnessDuration)
	{
		check(NumChannels > 0);
		check(SampleRate > 0.f);
		check(ShortTermLoudnessDuration > 0.f);

		Analyzer = MakeUnique<FMultichannelLoudnessAnalyzer>(InParams.SampleRate, InAnalyzerSettings.LoudnessAnalyzerSettings);

		int32 NumSlidingWindowFrames = Analyzer->GetSettings().WindowSize;

		int32 NumAnalysisFrames = FMath::CeilToInt(InAnalyzerSettings.AnalysisWindowDuration * InParams.SampleRate);
		int32 NumAnalysisHopFrames = FMath::CeilToInt(InAnalyzerSettings.AnalysisPeriod * InParams.SampleRate);

		// There may be multiple calls to the LoudnessAnalyzer to produce a single 
		// FLoudnessDatum. The sliding window hop is tuned here so that it best 
		// matches the desired AnalysisPeriod while maintaining between a 25% to 
		// 75% window overlap.
		LKFSUtils::TuneSlidingWindowHopSize(NumAnalysisHopFrames, NumSlidingWindowFrames, NumSlidingWindowHopFrames, NumAnalysisHopWindows);

		// Determine how many windows to analyze per loudness result
		NumAnalysisWindows = FMath::Max(1, ((NumAnalysisFrames - (NumSlidingWindowFrames / 2)) / NumSlidingWindowHopFrames) + 1);

		// Include NumChannels in calculating window size because windows are generated 
		// with interleaved samples and deinterleaved later. 
		int32 NumSlidingWindowSamples = NumSlidingWindowFrames * NumChannels;
		int32 NumSlidingWindowHopSamples = NumSlidingWindowHopFrames * NumChannels;

		InternalBuffer = MakeUnique<TSlidingBuffer<float>>(NumSlidingWindowSamples, NumSlidingWindowHopSamples);
		InternalWindow.AddUninitialized(NumSlidingWindowSamples);
		ChannelPerceptualEnergy.AddZeroed(NumChannels * NumAnalysisWindows);
		OverallPerceptualEnergy.AddZeroed(NumAnalysisWindows);
	}

	void FLKFSNRTWorker::Analyze(TArrayView<const float> InAudio, IAnalyzerNRTResult* OutResult) 
	{
		NumFrames += InAudio.Num() / NumChannels;

		// Assume that outer layers ensured that this is of correct type.
		FLKFSNRTResult* LoudnessResult = static_cast<FLKFSNRTResult*>(OutResult);

		check(nullptr != LoudnessResult);
		
		TAutoSlidingWindow<float> SlidingWindow(*InternalBuffer, InAudio, InternalWindow);

		AnalyzeSlidingWindow(SlidingWindow, *LoudnessResult);
	}

	// Called when analysis of audio asset is complete. 
	void FLKFSNRTWorker::Finalize(IAnalyzerNRTResult* OutResult) 
	{
		FLKFSNRTResult* LoudnessResult = static_cast<FLKFSNRTResult*>(OutResult);

		check(nullptr != LoudnessResult);

		TArray<float> EmptyArray;
		constexpr bool bFlushAudioWithZeroes = false;
		TAutoSlidingWindow<float> SlidingWindow(*InternalBuffer, EmptyArray, InternalWindow, bFlushAudioWithZeroes);

		AnalyzeSlidingWindow(SlidingWindow, *LoudnessResult);

		// Finalize all the channels now that we have finished analyzing audio.
		FinalizeChannels(*LoudnessResult);

		// Prepare result for fast lookup.
		LoudnessResult->SortChronologically(); 

		LoudnessResult->SetDurationInSeconds(static_cast<float>(NumFrames) / SampleRate);

		// Update integrated, gated and short term loudness. 
		CalculateAggregateLoudnessStats(*LoudnessResult);

		// Reset internal counters
		NumAnalyzedBuffers = 0; 
		NumFrames = 0;

		// Reset sliding buffer
		InternalBuffer->Reset();
	}

	void FLKFSNRTWorker::FinalizeChannels(FLKFSNRTResult& OutResult)
	{
		for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ChannelIdx++)
		{
			if (!OutResult.ContainsChannel(ChannelIdx))
			{
				OutResult.AddChannel(ChannelIdx);
			}
		}

		if (!OutResult.ContainsChannel(FLKFSNRTResult::ChannelIndexOverall))
		{
			OutResult.AddChannel(FLKFSNRTResult::ChannelIndexOverall);
		}
	}

	void FLKFSNRTWorker::AnalyzeSlidingWindow(TAutoSlidingWindow<float>& InSlidingWindow, FLKFSNRTResult& OutResult)
	{
		check(NumAnalysisWindows > 0);
		check(NumAnalysisHopWindows > 0);
		
		for (const TArray<float>& Window : InSlidingWindow)
		{
			AnalyzeWindow(Window, OutResult);

			if ((NumAnalyzedBuffers >= NumAnalysisWindows) && ((NumAnalyzedBuffers - NumAnalysisWindows) % NumAnalysisHopWindows) == 0)
			{
				AddCurrentLoudnessDataToResult(OutResult);
			}
		}
	}

	void FLKFSNRTWorker::AnalyzeWindow(TArrayView<const float> InWindow, FLKFSNRTResult& OutResult)
	{
		const int32 WrappedWindowIdx = (NumAnalyzedBuffers % NumAnalysisWindows);
		const int32 ChannelPerceptualEnergyWritePos = WrappedWindowIdx * NumChannels;

		// Calculate perceptual energy
		OverallPerceptualEnergy[WrappedWindowIdx] = Analyzer->CalculatePerceptualEnergy(InWindow, NumChannels, MakeArrayView<float>(ChannelPerceptualEnergy.GetData() + ChannelPerceptualEnergyWritePos, NumChannels));

		// update counters
		NumAnalyzedBuffers++;
	}

	void FLKFSNRTWorker::AddCurrentLoudnessDataToResult(FLKFSNRTResult& OutResult)
	{
		// Accounting for all the frames positions analyzed in creating the 
		// data, determine the center frame. 
		const int32 CenterFramePos = ((2 * NumAnalyzedBuffers - NumAnalysisWindows) * NumSlidingWindowHopFrames + Analyzer->GetSettings().WindowSize) / 2;
		const float Timestamp = CenterFramePos / SampleRate;

		// Compute loudness per channel
		for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ChannelIdx++)
		{
			// Determine average perceptual energy over analysis window
			float PerceptualEnergy = 0.f;
			for (int32 Pos = ChannelIdx; Pos < ChannelPerceptualEnergy.Num(); Pos += NumChannels)
			{
				PerceptualEnergy += ChannelPerceptualEnergy[Pos];
			}
			PerceptualEnergy /= (float)NumAnalysisWindows;

			// Convert to loudness.
			const float PerceptualLoudness = FLoudnessAnalyzer::ConvertPerceptualEnergyToLKFS(PerceptualEnergy);
			OutResult.Add(FLKFSNRTResults{.Channel=ChannelIdx, .Timestamp=Timestamp, .Energy=PerceptualEnergy, .Loudness=PerceptualLoudness});
		}

		{
			// Compute loudness overall
			float PerceptualEnergy = 0.f;
			for (int32 Pos = 0; Pos < OverallPerceptualEnergy.Num(); Pos++)
			{
				PerceptualEnergy += OverallPerceptualEnergy[Pos];
			}
			PerceptualEnergy /= (float)NumAnalysisWindows;

			const float PerceptualLoudness = FLoudnessAnalyzer::ConvertPerceptualEnergyToLKFS(PerceptualEnergy);
			OutResult.Add(FLKFSNRTResults{.Channel=FLKFSNRTResult::ChannelIndexOverall,  .Timestamp=Timestamp, .Energy=PerceptualEnergy, .Loudness=PerceptualLoudness});
		}
	}

	void FLKFSNRTWorker::CalculateAggregateLoudnessStats(FLKFSNRTResult& InOutResult) const
	{
		checkf(InOutResult.IsSortedChronologically(), TEXT("Results should be presorted in chronological order"));

		TArray<int32> Channels;
		InOutResult.GetChannels(Channels);

		for (int32 ChannelIndex : Channels)
		{
			FLKFSNRTAggregateStats AggregateStats;

			LKFSNRTFactoryPrivate::CalculateAggregateLoudnessStats(ShortTermLoudnessDuration, InOutResult.GetMutableChannelLoudnessArray(ChannelIndex), AggregateStats);

			InOutResult.SetAggregateLoudnessStats(ChannelIndex, AggregateStats);
		}
	}


	/************************************************************************/
	/************************* FLKFSNRTFactory **************************/
	/************************************************************************/

	/** Name of specific analyzer type. */
	FName FLKFSNRTFactory::GetName() const 
	{
		static FName FactoryName(TEXT("LKFSNRTFactory"));
		return FactoryName;
	}

	/** Human readable name of analyzer. */
	FString FLKFSNRTFactory::GetTitle() const
	{
		return TEXT("LKFS Analyzer Non-Real-Time");
	}

	TUniquePtr<IAnalyzerNRTResult> FLKFSNRTFactory::NewResult() const
	{
		TUniquePtr<FLKFSNRTResult> Result = MakeUnique<FLKFSNRTResult>();
		return Result;
	}

	TUniquePtr<IAnalyzerNRTWorker> FLKFSNRTFactory::NewWorker(const FAnalyzerNRTParameters& InParams, const IAnalyzerNRTSettings* InSettings) const
	{
		const FLKFSNRTSettings* LoudnessSettings = static_cast<const FLKFSNRTSettings*>(InSettings);

		check(nullptr != LoudnessSettings);

		return MakeUnique<FLKFSNRTWorker>(InParams, *LoudnessSettings);
	}
}

