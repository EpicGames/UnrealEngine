// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoudnessFactory.h"
#include "DSP/SlidingWindow.h"

namespace Audio
{
	// Use for overall loudness storage
	const int32 FLoudnessResult::ChannelIndexOverall = INDEX_NONE;

	void FLoudnessResult::Add(const FLoudnessEntry& InDatum)
	{
		// Store loudness data in appropriate channel
		TArray<FLoudnessEntry>& LoudnessArray = ChannelLoudnessArrays.FindOrAdd(InDatum.Channel);
		LoudnessArray.Add(InDatum);
	}

	const TArray<FLoudnessEntry>& FLoudnessResult::GetChannelLoudnessArray(int32 ChannelIdx) const
	{
		return ChannelLoudnessArrays[ChannelIdx];
	}

	const TArray<FLoudnessEntry>& FLoudnessResult::GetLoudnessArray() const
	{
		return ChannelLoudnessArrays[ChannelIndexOverall];
	}

	int32 FLoudnessResult::GetNumChannels() const 
	{
		if (ChannelLoudnessArrays.Num() > 0)
		{
			// Channel loudness arrays have -1 reserved for overall loudness
			return ChannelLoudnessArrays.Num() - 1;
		}
		return 0;
	}

	FLoudnessWorker::FLoudnessWorker(const FAnalyzerParameters& InParams, const FLoudnessSettings& InAnalyzerSettings)
		: NumChannels(InParams.NumChannels)
		, NumAnalyzedBuffers(0)
		, NumHopFrames(0)
		, SampleRate(InParams.SampleRate)
	{
		check(NumChannels > 0);
		check(SampleRate > 0);

		// Include NumChannels in calculating window size because windows are generated 
		// with interleaved samples and deinterleaved later. 
		int32 NumWindowSamples = InAnalyzerSettings.FFTSize * NumChannels;

		NumHopFrames = FMath::CeilToInt(InAnalyzerSettings.AnalysisPeriod * InParams.SampleRate);
		NumHopFrames = FMath::Max(1, NumHopFrames);

		int32 NumHopSamples = NumHopFrames * NumChannels;

		InternalBuffer = MakeUnique<TSlidingBuffer<float>>(NumWindowSamples, NumHopSamples);

		Analyzer = MakeUnique<FMultichannelLoudnessAnalyzer>(InParams.SampleRate, InAnalyzerSettings);
	}

	void FLoudnessWorker::Analyze(TArrayView<const float> InAudio, IAnalyzerResult* OutResult) 
	{
		FLoudnessResult* LoudnessResult = static_cast<FLoudnessResult*>(OutResult);
		check(nullptr != LoudnessResult);

		TAutoSlidingWindow<float> SlidingWindow(*InternalBuffer, InAudio, InternalWindow);

		// Loop through entire array of input samples. 
		for (const TArray<float>& Window : SlidingWindow)
		{
			AnalyzeWindow(Window, *LoudnessResult);
		}
	}

	void FLoudnessWorker::AnalyzeWindow(TArrayView<const float> InWindow, FLoudnessResult& OutResult)
	{
		TArray<float> ChannelPerceptualEnergy;
		ChannelPerceptualEnergy.AddZeroed(NumChannels);
		float OverallPerceptualEnergy = 0.0f;

		// Calculate perceptual energy
		OverallPerceptualEnergy = Analyzer->CalculatePerceptualEnergy(InWindow, NumChannels, TArrayView<float>(ChannelPerceptualEnergy));

		// Convert to perceptual energy to loudness
		TArray<float> ChannelLoudness;
		float OverallLoudness = 0.f;

		OverallLoudness = FLoudnessAnalyzer::ConvertPerceptualEnergyToLoudness(OverallPerceptualEnergy);

		for (float PerceptualEnergy : ChannelPerceptualEnergy)
		{
			ChannelLoudness.Add(FLoudnessAnalyzer::ConvertPerceptualEnergyToLoudness(PerceptualEnergy));
		}

		// Calculate timestamp
		float Timestamp = ((NumAnalyzedBuffers * NumHopFrames) + (0.5f * Analyzer->GetSettings().FFTSize)) / SampleRate;

		// Add overall FLoudnessDatum to result
		const FLoudnessEntry OverallLoudnessEntry = { FLoudnessResult::ChannelIndexOverall, Timestamp, OverallPerceptualEnergy, OverallLoudness };
		OutResult.Add(OverallLoudnessEntry);

		// Add channel FLoudnessDatum to result
		for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ChannelIdx++)
		{
			const FLoudnessEntry ChannelEntry = { ChannelIdx, Timestamp, ChannelPerceptualEnergy[ChannelIdx], ChannelLoudness[ChannelIdx] };

			OutResult.Add(ChannelEntry);
		}

		NumAnalyzedBuffers++;
	}

	FName FLoudnessFactory::GetName() const 
	{
		static FName FactoryName(TEXT("LoudnessFactory"));
		return FactoryName;
	}

	FString FLoudnessFactory::GetTitle() const
	{
		return TEXT("Real-Time Loudness Analyzer");
	}

	TUniquePtr<IAnalyzerResult> FLoudnessFactory::NewResult() const
	{
		TUniquePtr<FLoudnessResult> Result = MakeUnique<FLoudnessResult>();
		return Result;
	}

	TUniquePtr<IAnalyzerWorker> FLoudnessFactory::NewWorker(const FAnalyzerParameters& InParams, const IAnalyzerSettings* InSettings) const
	{
		const FLoudnessSettings* LoudnessSettings = static_cast<const FLoudnessSettings*>(InSettings);

		check(nullptr != LoudnessSettings);

		return MakeUnique<FLoudnessWorker>(InParams, *LoudnessSettings);
	}
}

