// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoudnessNRTFactory.h"

#include "Algo/IsSorted.h"
#include "AudioSynesthesiaCoreLog.h"
#include "AudioSynesthesiaCustomVersion.h"
#include "DSP/SlidingWindow.h"

namespace Audio
{
	/************************************************************************/
	/********************** FLoudnessNRTResult ******************************/
	/************************************************************************/
	FArchive &operator <<(FArchive& Ar, FLoudnessDatum& Datum)
	{
		Ar.UsingCustomVersion(FAudioSynesthesiaCustomVersion::GUID);

		Ar << Datum.Channel;
		Ar << Datum.Timestamp;
		Ar << Datum.Energy;
		Ar << Datum.Loudness;

		return Ar;
	}

	// Use for overall loudness storage
	const int32 FLoudnessNRTResult::ChannelIndexOverall = -1;


	FLoudnessNRTResult::FLoudnessNRTResult()
	:	DurationInSeconds(0.f)
	,	bIsSortedChronologically(false)
	{
	}

	void FLoudnessNRTResult::Serialize(FArchive& Archive)
	{
		Archive.UsingCustomVersion(FAudioSynesthesiaCustomVersion::GUID);

		Archive << DurationInSeconds;
		Archive << bIsSortedChronologically;
		Archive << ChannelLoudnessArrays;
		Archive << ChannelLoudnessIntervals;
	}

	void FLoudnessNRTResult::Add(const FLoudnessDatum& InDatum)
	{
		// Store loudness data in appropriate channel
		TArray<FLoudnessDatum>& LoudnessArray = ChannelLoudnessArrays.FindOrAdd(InDatum.Channel);
		LoudnessArray.Add(InDatum);

		// Update intervals per channel
		FFloatInterval& Interval = ChannelLoudnessIntervals.FindOrAdd(InDatum.Channel);
		Interval.Include(InDatum.Loudness);

		// Mark as not sorted
		bIsSortedChronologically = false;
	}

	bool FLoudnessNRTResult::ContainsChannel(int32 InChannelIndex) const
	{
		return ChannelLoudnessArrays.Contains(InChannelIndex);
	}

	const TArray<FLoudnessDatum>& FLoudnessNRTResult::GetChannelLoudnessArray(int32 ChannelIdx) const
	{
		return ChannelLoudnessArrays[ChannelIdx];
	}

	const TArray<FLoudnessDatum>& FLoudnessNRTResult::GetLoudnessArray() const
	{
		return ChannelLoudnessArrays[ChannelIndexOverall];
	}

	float FLoudnessNRTResult::GetLoudnessRange(float InNoiseFloor) const
	{
		return GetChannelLoudnessRange(ChannelIndexOverall, InNoiseFloor);
	}

	float FLoudnessNRTResult::GetChannelLoudnessRange(int32 InChannelIdx, float InNoiseFloor) const
	{
		const FFloatInterval& Interval = ChannelLoudnessIntervals.FindRef(InChannelIdx);

		if (Interval.Contains(InNoiseFloor))
		{
			// Noise floor is greater than minimum, so range must be relative to noise floor
			return Interval.Max - InNoiseFloor;
		}
		else if (InNoiseFloor > Interval.Max)
		{
			return 0.f;
		}

		// Noise floor is less than minimum, so use entire range.
		return Interval.Size();
	}

	void FLoudnessNRTResult::GetChannels(TArray<int32>& OutChannels) const
	{
		ChannelLoudnessArrays.GetKeys(OutChannels);
	}

	float FLoudnessNRTResult::GetDurationInSeconds() const 
	{
		return DurationInSeconds;
	}

	void FLoudnessNRTResult::SetDurationInSeconds(float InDuration)
	{
		DurationInSeconds = InDuration;
	}

	bool FLoudnessNRTResult::IsSortedChronologically() const
	{
		return bIsSortedChronologically;
	}

	void FLoudnessNRTResult::SortChronologically()
	{
		for (auto& KeyValue : ChannelLoudnessArrays)
		{
			KeyValue.Value.Sort([](const FLoudnessDatum& A, const FLoudnessDatum& B) { return A.Timestamp < B.Timestamp; });
		}

		bIsSortedChronologically = true;
	}

	/************************************************************************/
	/********************** FLoudnessNRTWorker ******************************/
	/************************************************************************/

	FLoudnessNRTWorker::FLoudnessNRTWorker(const FAnalyzerNRTParameters& InParams, const FLoudnessNRTSettings& InAnalyzerSettings)
	: NumChannels(InParams.NumChannels)
	, NumAnalyzedBuffers(0)
	, NumHopFrames(0)
	, NumFrames(0)
	, SampleRate(InParams.SampleRate)
	{
		check(NumChannels > 0);
		check(SampleRate > 0.f);

		// Include NumChannels in calculating window size because windows are generated 
		// with interleaved samples and deinterleaved later. 
		int32 NumWindowSamples = InAnalyzerSettings.FFTSize * NumChannels;

		NumHopFrames = FMath::CeilToInt(InAnalyzerSettings.AnalysisPeriod * InParams.SampleRate);

		NumHopFrames = FMath::Max(1, NumHopFrames);

		int32 NumHopSamples = NumHopFrames * NumChannels;

		InternalBuffer = MakeUnique<TSlidingBuffer<float>>(NumWindowSamples, NumHopSamples);

		Analyzer = MakeUnique<FMultichannelLoudnessAnalyzer>(InParams.SampleRate, InAnalyzerSettings);
	}

	void FLoudnessNRTWorker::Analyze(TArrayView<const float> InAudio, IAnalyzerNRTResult* OutResult) 
	{
		NumFrames += InAudio.Num() / NumChannels;

		// Assume that outer layers ensured that this is of correct type.
		FLoudnessNRTResult* LoudnessResult = static_cast<FLoudnessNRTResult*>(OutResult);

		check(nullptr != LoudnessResult);
		
		TAutoSlidingWindow<float> SlidingWindow(*InternalBuffer, InAudio, InternalWindow);

		// Loop through entire array of input samples. 
		for (const TArray<float>& Window : SlidingWindow)
		{
			AnalyzeWindow(Window, *LoudnessResult);
		}
	}

	// Called when analysis of audio asset is complete. 
	void FLoudnessNRTWorker::Finalize(IAnalyzerNRTResult* OutResult) 
	{
		FLoudnessNRTResult* LoudnessResult = static_cast<FLoudnessNRTResult*>(OutResult);

		check(nullptr != LoudnessResult);

		TArray<float> EmptyArray;
		bool bDoFlush = true; // Add zeros to last window until all data has been processed.
		TAutoSlidingWindow<float> SlidingWindow(*InternalBuffer, EmptyArray, InternalWindow, bDoFlush);

		// Loop through entire array of input samples. 
		for (const TArray<float>& Window : SlidingWindow)
		{
			AnalyzeWindow(Window, *LoudnessResult);
		}

		// Prepare result for fast lookup.
		LoudnessResult->SortChronologically(); 

		LoudnessResult->SetDurationInSeconds(static_cast<float>(NumFrames) / SampleRate);

		// Reset internal counters
		NumAnalyzedBuffers = 0; 
		NumFrames = 0;

		// Reset sliding buffer
		InternalBuffer->Reset();
	}

	void FLoudnessNRTWorker::AnalyzeWindow(TArrayView<const float> InWindow, FLoudnessNRTResult& OutResult)
	{
		TArray<float> ChannelPerceptualEnergy;
		ChannelPerceptualEnergy.AddUninitialized(NumChannels);
		float OverallPerceptualEnergy = 0.0f;

		// Calculate perceptual energy
		OverallPerceptualEnergy = Analyzer->CalculatePerceptualEnergy(InWindow, NumChannels, TArrayView<float>(ChannelPerceptualEnergy));

		// Convert to perceptual energy to loudness
		TArray<float> ChannelLoudness;
		float OverallLoudness = 0.f;
		
		OverallLoudness = FLoudnessAnalyzer::ConvertPerceptualEnergyToLoudness(OverallPerceptualEnergy);

		for (float PerceptualEnergy : ChannelPerceptualEnergy)
		{
			ChannelLoudness.Add(FLoudnessAnalyzer::ConvertPerceptualEnergyToLoudness(PerceptualEnergy));
		}

		// Calculate timestamp
		float Timestamp = ((NumAnalyzedBuffers * NumHopFrames) + (0.5f * Analyzer->GetSettings().FFTSize)) / SampleRate;

		// Add overall FLoudnessDatum to result
		const FLoudnessDatum OverallDatum = {FLoudnessNRTResult::ChannelIndexOverall, Timestamp, OverallPerceptualEnergy, OverallLoudness};
		OutResult.Add(OverallDatum);

		// Add channel FLoudnessDatum to result
		for (int32 ChannelIdx = 0; ChannelIdx < NumChannels; ChannelIdx++)
		{
			const FLoudnessDatum ChannelDatum = {ChannelIdx, Timestamp, ChannelPerceptualEnergy[ChannelIdx], ChannelLoudness[ChannelIdx]};

			OutResult.Add(ChannelDatum);
		}	

		// update counters
		NumAnalyzedBuffers++;
	}


	/************************************************************************/
	/************************* FLoudnessNRTFactory **************************/
	/************************************************************************/

	/** Name of specific analyzer type. */
	FName FLoudnessNRTFactory::GetName() const 
	{
		static FName FactoryName(TEXT("LoudnessNRTFactory"));
		return FactoryName;
	}

	/** Human readable name of analyzer. */
	FString FLoudnessNRTFactory::GetTitle() const
	{
		return TEXT("Loudness Analyzer Non-Real-Time");
	}

	TUniquePtr<IAnalyzerNRTResult> FLoudnessNRTFactory::NewResult() const
	{
		TUniquePtr<FLoudnessNRTResult> Result = MakeUnique<FLoudnessNRTResult>();
		return Result;
	}

	TUniquePtr<IAnalyzerNRTWorker> FLoudnessNRTFactory::NewWorker(const FAnalyzerNRTParameters& InParams, const IAnalyzerNRTSettings* InSettings) const
	{
		const FLoudnessNRTSettings* LoudnessSettings = static_cast<const FLoudnessNRTSettings*>(InSettings);

		check(nullptr != LoudnessSettings);

		return MakeUnique<FLoudnessNRTWorker>(InParams, *LoudnessSettings);
	}
}

