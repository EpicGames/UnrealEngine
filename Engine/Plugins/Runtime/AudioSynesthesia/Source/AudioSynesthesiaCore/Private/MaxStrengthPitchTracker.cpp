// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaxStrengthPitchTracker.h"

#include "Algo/MaxElement.h"
#include "AudioSynesthesiaCoreLog.h"

namespace Audio
{
	namespace MaxStrengthPitchTrackIntrinsics
	{

		// Likely this method will be useful elsewhere
		template<typename T, class PROJECTION_CLASS, class ON_GROUP_CLASS>
		void GroupBy(TArrayView<const T> InData, const PROJECTION_CLASS& Proj, const ON_GROUP_CLASS& OnGroup)
		{
			const int32 Num = InData.Num();

			if (Num > 0)
			{
				const T* Data = InData.GetData();

				int32 GroupStartIndex = 0;
				auto Key = Proj(Data[0]);

				for (int32 i = 1; i < Num; i++)
				{
					auto NewKey = Proj(Data[i]);
					if (Key != NewKey)
					{
						OnGroup(Key, InData.Slice(GroupStartIndex, i - GroupStartIndex));

						Key = NewKey;
						GroupStartIndex = i;
					}
				}

				OnGroup(Key, InData.Slice(GroupStartIndex, Num - GroupStartIndex));
			}
		}
	}

	FMaxStrengthPitchTracker::FMaxStrengthPitchTracker(const FMaxStrengthPitchTrackerSettings& InSettings, TUniquePtr<IPitchDetector> InDetector)
	:	Settings(InSettings)
	,	PitchDetector(MoveTemp(InDetector))
	{
		UE_CLOGF(!PitchDetector.IsValid(), LogAudioSynesthesiaCore, Warning, "Invalid pointer to pitch detector. MaxStrengthPitchDetector will not return valid results.");
	}

	FMaxStrengthPitchTracker::~FMaxStrengthPitchTracker()
	{
	}

	void FMaxStrengthPitchTracker::TrackPitches(const FAlignedFloatBuffer& InMonoAudio, TArray<FPitchTrackInfo>& OutPitchTracks)
	{
		if (PitchDetector.IsValid())
		{
			TArray<FPitchInfo> NewObservations;

			PitchDetector->DetectPitches(InMonoAudio, NewObservations);

			Observations.Append(MoveTemp(NewObservations));
		}
	}

	void FMaxStrengthPitchTracker::Finalize(TArray<FPitchTrackInfo>& OutPitchTracks)
	{
		using namespace MaxStrengthPitchTrackIntrinsics;

		if (PitchDetector.IsValid())
		{
			PitchDetector->Finalize(Observations);
		}

		// Sort obersvations temporally
		Observations.Sort([](const FPitchInfo& A, const FPitchInfo& B) { return A.Timestamp < B.Timestamp; });

		TArray<FPitchInfo> MaxObservations;

		TArrayView<const FPitchInfo> ObservationsView(Observations);

		// Group by pitch infos with equal timestamps
		GroupBy(ObservationsView, [](const FPitchInfo& Info) { return Info.Timestamp; }, [&](float Timestamp, TArrayView<const FPitchInfo> PitchInfos)
			{
				// Get maximum strength element within all elements with equal timestamps
				const FPitchInfo* MaxInfo = Algo::MaxElementBy(PitchInfos, [](const FPitchInfo& Info) { return Info.Strength; });

				if (nullptr != MaxInfo)
				{
					MaxObservations.Add(*MaxInfo);
				}
			}
		);

		// Build tracks if requirements are met.
		FPitchTrackInfo TrackInfo;

		for (const FPitchInfo& PitchInfo : MaxObservations)
		{
			bool bEndPreviousTrack = false;

			bool bAddToTrack = PitchInfo.Strength >= Settings.MinimumStrength;

			if (TrackInfo.Observations.Num() > 0)
			{
				if (!bAddToTrack)
				{
					// End previous track if new pitch observations is too weak.
					bEndPreviousTrack = true;
				}
				else
				{
					const FPitchInfo& Last = TrackInfo.Observations.Last(0);
					if (Last.Frequency == 0.f)
					{
						// A frequency of zero screws up the frequency deviation calculation. Also, DC is not a pitch.
						bEndPreviousTrack = true;
					}
					else
					{
						// Check if frequency ratio deviation below threshold
						float FrequencyRatioDeviation = FMath::Abs(1.f - PitchInfo.Frequency / Last.Frequency);
						bEndPreviousTrack |= (FrequencyRatioDeviation > Settings.MaximumFrequencyRatioDeviation);
					}
				}
			}

			if (bEndPreviousTrack)
			{
				OutPitchTracks.Add(MoveTemp(TrackInfo));

				TrackInfo.Observations.Reset();
			}

			if (bAddToTrack)
			{
				TrackInfo.Observations.Add(PitchInfo);
			}
		}

		if (TrackInfo.Observations.Num() > 0)
		{
			OutPitchTracks.Add(MoveTemp(TrackInfo));
		}

		// Clear out observations for next audio stream.
		Observations.Reset();
	}
}
