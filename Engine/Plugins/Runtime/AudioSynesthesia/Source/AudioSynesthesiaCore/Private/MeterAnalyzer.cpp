// Copyright Epic Games, Inc. All Rights Reserved.


#include "MeterAnalyzer.h"


namespace Audio
{
	FMeterAnalyzer::FMeterAnalyzer(float InSampleRate, int32 InNumChannels, const FMeterAnalyzerSettings& InSettings)
	: EnvelopeFollower(FEnvelopeFollowerInitParams{InSampleRate, InNumChannels, static_cast<float>(InSettings.MeterAttackTime), static_cast<float>(InSettings.MeterReleaseTime), InSettings.MeterPeakMode, InSettings.bIsAnalog })
	, Settings(InSettings)
	, SampleRate(InSampleRate)
	, NumChannels(InNumChannels)
	{
		if (!ensure(NumChannels > 0))
		{
			NumChannels = 1;
		}
		if (!ensure(SampleRate > 0.f))
		{
			SampleRate = 48000.f;
		}

		EnvelopeDataPerChannel.AddDefaulted(NumChannels);
		ClippingDataPerChannel.AddDefaulted(NumChannels);
		PeakHoldFrames = FMath::Max(1, FMath::RoundToInt(static_cast<float>(InSettings.PeakHoldTime) * SampleRate / 1000.f));
	}

	FMeterAnalyzerResults FMeterAnalyzer::ProcessAudio(TArrayView<const float> InSampleView)
	{
		// Feed audio through the envelope followers
		const float* InData = InSampleView.GetData();
		const int32 NumSamples = InSampleView.Num();
		const int32 NumFrames = NumSamples / NumChannels;

		check((NumSamples % NumChannels) == 0);

		// Reset the clipping data for next block of audio
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
		{
			ClippingDataPerChannel[ChannelIndex].ClippingValue = 0.0f;
			ClippingDataPerChannel[ChannelIndex].NumSamplesClipping = 0;
		}

		for (int32 i = 0; i < NumSamples; i++)
		{
			if (FMath::Abs(InData[i]) > Settings.ClippingThreshold)
			{
				const int32 ChannelIndex = i % NumChannels;
				FClippingData& ClippingData = ClippingDataPerChannel[ChannelIndex];

				// Track how many samples clipped in this block
				ClippingData.NumSamplesClipping++;

				// Track the max clipping value in this block of audio
				ClippingData.ClippingValue = FMath::Max(ClippingData.ClippingValue, InData[i]);
			}
		}

		EnvelopeBuffer.Reset(NumSamples);
		EnvelopeBuffer.AddUninitialized(NumSamples);
		EnvelopeFollower.ProcessAudio(InData, NumFrames, EnvelopeBuffer.GetData());

		// Find peaks in the envelope.
		const float* Envelope = EnvelopeBuffer.GetData();
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ChannelIndex++)
		{
			FEnvelopeData& EnvelopeData = EnvelopeDataPerChannel[ChannelIndex];
			// Reset max envelope data. 
			EnvelopeData.MaxEnvelopeValue = 0.f;
			if (EnvelopeData.FramesUntilPeakReset < 1)
			{
				EnvelopeData.PeakValue = 0;
			}

			// Temp variables to hold onto prior frame's envelope info.
			float PriorValue = EnvelopeData.PriorEnvelopeValue;
			bool bPriorSlopeIsPositive = EnvelopeData.bPriorEnvelopeSlopeIsPositive;

			for (int32 SampleIndex = ChannelIndex; SampleIndex < NumSamples; SampleIndex += NumChannels)
			{
				// Get maximum value in envelope.
				EnvelopeData.MaxEnvelopeValue = FMath::Max(Envelope[SampleIndex], EnvelopeData.MaxEnvelopeValue);

				// Positive if we're rising
				bool bNewSlopeIsNegative = (Envelope[SampleIndex] - PriorValue) >= 0.0f;
				bool bIsPeak = bNewSlopeIsNegative && bPriorSlopeIsPositive;

				// Detect if the new peak envelope value is less than the previous peak envelope value (i.e. local maximum)
				if (bIsPeak)
				{
					const int32 FrameIndex = SampleIndex / NumChannels;

					// Check if the peak value is larger than the current peak value or if enough time has elapsed to store a new peak
					if ((EnvelopeData.PeakValue < Envelope[SampleIndex]) || (FrameIndex > EnvelopeData.FramesUntilPeakReset))
					{
						EnvelopeData.PeakValue = Envelope[SampleIndex];
						EnvelopeData.FramesUntilPeakReset = FrameIndex + PeakHoldFrames;
					}
				}

				// Update the current peak envelope data to the new value
				PriorValue = Envelope[SampleIndex];
				bPriorSlopeIsPositive = !bNewSlopeIsNegative;
			}

			// Save for next call
			EnvelopeData.PriorEnvelopeValue = PriorValue;
			EnvelopeData.bPriorEnvelopeSlopeIsPositive = bPriorSlopeIsPositive;

			if (EnvelopeData.FramesUntilPeakReset > 0)
			{
				// Decrement frame counters for next call.
				EnvelopeData.FramesUntilPeakReset -= NumFrames;
			}
		}

		// Build the results data
		FMeterAnalyzerResults Results;

		FrameCounter += NumFrames;
		Results.TimeSec = static_cast<float>(FrameCounter) / SampleRate;

		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
		{
			// Add clipping data
			FClippingData& ClippingData = ClippingDataPerChannel[ChannelIndex];

			Results.ClippingValues.Add(ClippingData.ClippingValue);
			Results.NumSamplesClipping.Add(ClippingData.NumSamplesClipping);

			// update sample count of the env data
			const FEnvelopeData& EnvData = EnvelopeDataPerChannel[ChannelIndex];

			Results.MeterValues.Add(EnvData.MaxEnvelopeValue);
			Results.PeakValues.Add(EnvData.PeakValue);
		}

		return Results;
	}

	const FMeterAnalyzerSettings& FMeterAnalyzer::GetSettings() const
	{
		return Settings;
	}
}
