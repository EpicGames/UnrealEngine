// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeterFactory.h"
#include "DSP/SlidingWindow.h"

namespace Audio
{
	// Use for overall meter storage
	const int32 FMeterResult::ChannelIndexOverall = INDEX_NONE;

	void FMeterResult::Add(const FMeterEntry& InEntry)
	{
		// Store meter data in appropriate channel
		TArray<FMeterEntry>& MeterArray = ChannelMeterArrays.FindOrAdd(InEntry.Channel);
		MeterArray.Add(InEntry);
	}

	const TArray<FMeterEntry>& FMeterResult::GetChannelMeterArray(int32 ChannelIdx) const
	{
		return ChannelMeterArrays[ChannelIdx];
	}

	const TArray<FMeterEntry>& FMeterResult::GetMeterArray() const
	{
		return ChannelMeterArrays[ChannelIndexOverall];
	}

	int32 FMeterResult::GetNumChannels() const 
	{
		if (ChannelMeterArrays.Num() > 0)
		{
			// Channel meter arrays have -1 reserved for overall meter results
			return ChannelMeterArrays.Num() - 1;
		}
		return 0;
	}

	FMeterWorker::FMeterWorker(const FAnalyzerParameters& InParams, const FMeterSettings& InAnalyzerSettings)
		: NumChannels(InParams.NumChannels)
		, NumAnalyzedBuffers(0)
		, SampleRate(InParams.SampleRate)
	{
		check(NumChannels > 0);
		check(SampleRate > 0);

		NumHopFrames = FMath::CeilToInt(InAnalyzerSettings.AnalysisPeriod * (float)InParams.SampleRate);
		NumHopFrames = FMath::Max(1, NumHopFrames);

		float NumAnalysisFrames = InAnalyzerSettings.AnalysisPeriod * (float)SampleRate;
		int32 NumWindowSamples = NumAnalysisFrames * NumChannels;
		int32 NumHopSamples = NumHopFrames * NumChannels;

		InternalBuffer = MakeUnique<TSlidingBuffer<float>>(NumWindowSamples, NumHopSamples);
		MeterAnalyzer = MakeUnique<FMeterAnalyzer>(InParams.SampleRate, NumChannels, InAnalyzerSettings);
	}

	void FMeterWorker::Analyze(TArrayView<const float> InAudio, IAnalyzerResult* OutResult) 
	{
		FMeterResult* MeterResult = static_cast<FMeterResult*>(OutResult);
		check(MeterResult != nullptr);

		TAutoSlidingWindow<float> SlidingWindow(*InternalBuffer, InAudio, InternalWindow);

		// Loop through entire array of input samples. 
		for (const TArray<float>& Window : SlidingWindow)
		{
			const FMeterAnalyzerResults& Results = MeterAnalyzer->ProcessAudio(Window);

			float OverallMeter = 0.0f;
			float OverallPeak = 0.0f;
			float Timestamp = 0.0f;
			int32 MaxClippingSamples = 0;
			float MaxClippingValue = 0.0f;

			// Add the per-channel results
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
				FMeterEntry NewEntry = 
				{ 
					ChannelIndex, 
					Results.TimeSec, 
					Results.MeterValues[ChannelIndex], 
					Results.PeakValues[ChannelIndex],
					Results.ClippingValues[ChannelIndex],
					Results.NumSamplesClipping[ChannelIndex]
				};

				Timestamp = Results.TimeSec;

				// Add the entries to compute overall
				OverallMeter += NewEntry.MeterValue;
				OverallPeak += NewEntry.PeakValue;

				// The "overall" clipping data will be the max number of samples clipped between channels and the max clipping value
				MaxClippingSamples = FMath::Max(MaxClippingSamples, NewEntry.NumSamplesClipping);
				MaxClippingValue = FMath::Max(MaxClippingValue, NewEntry.ClippingValue);

				MeterResult->Add(NewEntry);
			}

			// Divide the overall meter and peak by number of channels to get average
			OverallMeter /= NumChannels;
			OverallPeak /= NumChannels;

			// Add to results
			FMeterEntry OverallEntry = 
			{ 
				FMeterResult::ChannelIndexOverall, 
				Timestamp, 
				OverallMeter, 
				OverallPeak, 
				MaxClippingValue,
				MaxClippingSamples
			};
			MeterResult->Add(OverallEntry);
		}
	}

	FName FMeterFactory::GetName() const 
	{
		static FName FactoryName(TEXT("MeterFactory"));
		return FactoryName;
	}

	FString FMeterFactory::GetTitle() const
	{
		return TEXT("Real-Time Meter Analyzer");
	}

	TUniquePtr<IAnalyzerResult> FMeterFactory::NewResult() const
	{
		TUniquePtr<FMeterResult> Result = MakeUnique<FMeterResult>();
		return Result;
	}

	TUniquePtr<IAnalyzerWorker> FMeterFactory::NewWorker(const FAnalyzerParameters& InParams, const IAnalyzerSettings* InSettings) const
	{
		const FMeterSettings* MeterSettings = static_cast<const FMeterSettings*>(InSettings);

		check(nullptr != MeterSettings);

		return MakeUnique<FMeterWorker>(InParams, *MeterSettings);
	}
}

