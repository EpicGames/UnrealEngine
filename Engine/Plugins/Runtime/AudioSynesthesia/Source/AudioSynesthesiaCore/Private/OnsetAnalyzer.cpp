// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnsetAnalyzer.h"
#include "PeakPicker.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	FOnsetStrengthAnalyzer::FOnsetStrengthAnalyzer(const FOnsetStrengthSettings& InSettings, float InSampleRate)
	:	Settings(InSettings)
	,	LagSpectraIndex(0)
	,	SlidingBuffer(InSettings.NumWindowFrames, InSettings.NumHopFrames)
	,	Window(Settings.WindowType, Settings.NumWindowFrames, 1, false)
	{
		// Check values legit.
		check(InSampleRate > 0.f);
		check(Settings.FFTSize > 0);
		check(Settings.NumHopFrames > 0);
		check(Settings.NumWindowFrames > 0);
		check(Settings.ComparisonLag > 0);
		check(Settings.MelSettings.NumBands > 0);
		check(Settings.NumWindowFrames <= Settings.FFTSize);

		FFFTSettings FFTSettings;
		FFTSettings.Log2Size = CeilLog2(Settings.FFTSize);
		FFTSettings.bArrays128BitAligned = true;
		FFTSettings.bEnableHardwareAcceleration = true;

		ActualFFTSize = 1 << FFTSettings.Log2Size;

		checkf(FFFTFactory::AreFFTSettingsSupported(FFTSettings), TEXT("No fft algorithm supports fft settings."));
		FFT = FFFTFactory::NewFFTAlgorithm(FFTSettings);

		if (FFT.IsValid())
		{
			// intialize buffers dealing with audio window
			WorkingBuffer.AddUninitialized(Settings.NumWindowFrames);

			// initialize buffers dealing with fft input / output
			WindowedSamples.AddZeroed(FFT->NumInputFloats());
			ComplexSpectrum.AddUninitialized(FFT->NumOutputFloats());
			RealSpectrum.AddUninitialized(FFT->NumOutputFloats() / 2);
			
			// intialize buffers dealing with perceptual spectrum 
			PreviousMelSpectra.AddDefaulted(Settings.ComparisonLag);
			for (FAlignedFloatBuffer& Spectrum : PreviousMelSpectra)
			{
				Spectrum.Reset();
				Spectrum.AddZeroed(Settings.MelSettings.NumBands);
			}

			MelSpectrum.AddUninitialized(Settings.MelSettings.NumBands);
			MelSpectrumDifference.AddUninitialized(Settings.MelSettings.NumBands);

			// create mel spectrum transform
			MelTransform = NewMelSpectrumKernelTransform(Settings.MelSettings, Settings.FFTSize, InSampleRate);
		}
	}

	void FOnsetStrengthAnalyzer::CalculateOnsetStrengths(TArrayView<const float> InSamples, TArray<float>& OutEnvelopeStrengths)
	{
		const bool bDoFlush = false;
		AnalyzeAudio(InSamples, OutEnvelopeStrengths, bDoFlush);
	}

	void FOnsetStrengthAnalyzer::FlushAudio(TArray<float>& OutEnvelopeStrengths)
	{
		TArray<float> EmptyArray;

		// Force flushing of internal audio buffer
		const bool bDoFlush = true;

		AnalyzeAudio(EmptyArray, OutEnvelopeStrengths, bDoFlush);
	}
	

	void FOnsetStrengthAnalyzer::Reset()
	{
		// Reset internal memory
		for (FAlignedFloatBuffer& Spectrum : PreviousMelSpectra)
		{
			Spectrum.Reset();
			Spectrum.AddZeroed(Settings.MelSettings.NumBands);
		}
		SlidingBuffer.Reset();
	}

	float FOnsetStrengthAnalyzer::GetTimestampForIndex(const FOnsetStrengthSettings& InSettings, float InSampleRate, int32 InIndex)
	{
		check(InSampleRate > 0.f);

		// Getting the exact timestamp of an onset from the onset strength envelope is non-trivial because
		// the onset strength envelope changes depending on the attack, decay and sustain of the onset as well
		// as the choice of window, window size, etc. etc. 
		//
		// This approach does not attempt to account for all the subtle variability than can come up, but 
		// uses a generally sane approach of estimating the timestamp to be somewhere between the center of
		// the "loud" window and the end of the "quiet" window.
		const int32 WindowOffset = InSettings.NumWindowFrames / 2;

		const int32 LoudFrameIndex = (InIndex * InSettings.NumHopFrames) + WindowOffset;

		// Get the end of the window to which this was compared.
		const int32 QuietIndex = InIndex - InSettings.ComparisonLag;
		const int32 QuietFrameIndex = (QuietIndex * InSettings.NumHopFrames) + InSettings.NumWindowFrames;

		const int32 EstimatedOnsetFrameIndex = (LoudFrameIndex + QuietFrameIndex) / 2;

		return static_cast<float>(EstimatedOnsetFrameIndex) / InSampleRate;
	}

	void FOnsetStrengthAnalyzer::AnalyzeAudio(TArrayView<const float> InSamples, TArray<float>& OutEnvelopeStrengths, bool bDoFlush)
	{
		typedef TAutoSlidingWindow<float, FAudioBufferAlignedAllocator> FSlidingWindow;

		// Going to fill OutEnvelopeStrengths with envelope strength values generated during this call.
		OutEnvelopeStrengths.Reset();

		// Slide over audio 
		FSlidingWindow SlidingWindow(SlidingBuffer, InSamples, WorkingBuffer, bDoFlush);
		for (const FAlignedFloatBuffer& WindowBuffer : SlidingWindow)
		{
			// Calculate onset strength per audio buffer.
			float OnsetStrength = GetNextOnsetStrength(WindowBuffer);

			OutEnvelopeStrengths.Add(OnsetStrength);
		}
	}

	float FOnsetStrengthAnalyzer::GetNextOnsetStrength(TArrayView<const float> InSamples)
	{
		float OnsetStrength = 0.f;

		if (FFT.IsValid() && MelTransform.IsValid())
		{
			// Copy input samples and apply window
			FMemory::Memcpy(WindowedSamples.GetData(), InSamples.GetData(), sizeof(float) * Settings.NumWindowFrames);
			Window.ApplyToBuffer(WindowedSamples.GetData());

			FFT->ForwardRealToComplex(WindowedSamples.GetData(), ComplexSpectrum.GetData());
			ArrayComplexToPower(ComplexSpectrum, RealSpectrum);
		
			ScalePowerSpectrumInPlace(ActualFFTSize, FFT->ForwardScaling(), EFFTScaling::None, RealSpectrum);
		
			// Convert spectrum to mel spectrum
			MelTransform->TransformArray(RealSpectrum, MelSpectrum);

			// Apply decibel scaling 
			ArrayPowerToDecibelInPlace(MelSpectrum, -90.f);

			// Clamp to noise floor
			ArrayClampMinInPlace(MelSpectrum, Settings.NoiseFloorDb);

			// Onset strength is mean(abs(diff(spectrum - previous_spectrum)))
			FAlignedFloatBuffer& PreviousSpectrum = PreviousMelSpectra[LagSpectraIndex];
			if (PreviousSpectrum.Num() > 0)
			{
				ArraySubtract(MelSpectrum, PreviousSpectrum, MelSpectrumDifference);

				// Half wave rectify
				ArrayClampMinInPlace(MelSpectrumDifference, 0.f);

				ArrayMean(MelSpectrumDifference, OnsetStrength);
			}

			// Save this spectrum for later use
			FMemory::Memcpy(PreviousSpectrum.GetData(), MelSpectrum.GetData(), sizeof(float) * Settings.MelSettings.NumBands);

			// Increment index for comparison spectra
			LagSpectraIndex = (LagSpectraIndex + 1) % Settings.ComparisonLag;
		}

		return OnsetStrength;
	}


	/*******************************************************************************/
	/*******************************************************************************/
	/*******************************************************************************/

	void OnsetExtractIndices(const FPeakPickerSettings& InSettings, TArrayView<const float> InOnsetEnvelope, TArray<int32>& OutOnsetIndices)
	{
		const int32 Num = InOnsetEnvelope.Num();
		OutOnsetIndices.Reset();

		if (Num < 1)
		{
			return;
		}
		
		TArray<float> NormOnsets;

		// Onset strength is normalized between [0, 1.0] before picking peaks.
		ArrayMinMaxNormalize(InOnsetEnvelope, NormOnsets);

		FPeakPicker PeakPicker(InSettings);
		PeakPicker.PickPeaks(NormOnsets, OutOnsetIndices);
	}

	/*******************************************************************************/
	/*******************************************************************************/
	/*******************************************************************************/

	void OnsetBacktrackIndices(TArrayView<const float> InOnsetStrength, TArrayView<const int32> InOnsetIndices, TArray<int32>& OutBacktrackedOnsetIndices)
	{
		const int32 NumOnsets = InOnsetIndices.Num();
		const int32 NumOnsetStrength = InOnsetStrength.Num();

		OutBacktrackedOnsetIndices.Reset();
		
		if (NumOnsets < 1)
		{
			return;
		}

		OutBacktrackedOnsetIndices.AddUninitialized(NumOnsets);

		const int32* InOnsetIndicesData = InOnsetIndices.GetData();
		int32* OutOnsetIndicesData = OutBacktrackedOnsetIndices.GetData();
		const float* InOnsetStrengthData = InOnsetStrength.GetData();

		for (int32 i = 0; i < NumOnsets; i++)
		{

			int32 OnsetStartIndex = InOnsetIndicesData[i];
			check(OnsetStartIndex < NumOnsetStrength);

			while (OnsetStartIndex > 0)
			{
				if (InOnsetStrengthData[OnsetStartIndex] <= 0.f)
				{
					// No onset strength means the audio is steady state or silent.
					break;
				}
				else if (InOnsetStrengthData[OnsetStartIndex - 1] <= InOnsetStrengthData[OnsetStartIndex])
				{
					// The onest strength is still holding steady or decreasing.
					OnsetStartIndex--;
				}
				else
				{
					// Onset strength is no longer decreasing.
					break;
				}
			}

			OutOnsetIndicesData[i] = OnsetStartIndex;
		}
	}
}
