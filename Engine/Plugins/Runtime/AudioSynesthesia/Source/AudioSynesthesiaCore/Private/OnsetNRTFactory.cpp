// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnsetNRTFactory.h"
#include "AudioSynesthesiaCustomVersion.h"
#include "DSP/DeinterleaveView.h"
#include "DSP/FloatArrayMath.h"

namespace Audio 
{
	/*******************************************************************************/
	/*************************** FOnsetNRTResult *******************************/
	/*******************************************************************************/

	FArchive &operator <<(FArchive& Ar, FOnset& Frame)
	{
		Ar.UsingCustomVersion(FAudioSynesthesiaCustomVersion::GUID);

		Ar << Frame.Channel;
		Ar << Frame.Timestamp;
		Ar << Frame.Strength;

		return Ar;
	}

	FOnsetNRTResult::FOnsetNRTResult()
	:	DurationInSeconds(0.f)
	,	bIsSortedChronologically(false)
	{}

	void FOnsetNRTResult::Serialize(FArchive& Archive)
	{
		Archive.UsingCustomVersion(FAudioSynesthesiaCustomVersion::GUID);

		Archive << DurationInSeconds;
		Archive << bIsSortedChronologically;
		Archive << ChannelOnsets;
		Archive << ChannelOnsetStrengthIntervals;
	}

	void FOnsetNRTResult::AddOnset(int32 InChannelIndex, float InTimestamp, float InStrength)
	{
		// Add to onset arrays
		TArray<FOnset>& OnsetArray = ChannelOnsets.FindOrAdd(InChannelIndex);
		OnsetArray.Emplace(InChannelIndex, InTimestamp, InStrength);

		// Store min/max value
		FFloatInterval& Interval = ChannelOnsetStrengthIntervals.FindOrAdd(InChannelIndex);
		Interval.Include(InStrength);

		bIsSortedChronologically = false;
	}

	void FOnsetNRTResult::AddChannel(int32 InChannelIndex)
	{
		// Add an empty channel. Useful when there is a channel with no onsets.
		ChannelOnsets.FindOrAdd(InChannelIndex);
		ChannelOnsetStrengthIntervals.FindOrAdd(InChannelIndex);
	}

	bool FOnsetNRTResult::ContainsChannel(int32 InChannelIndex) const
	{
		return ChannelOnsets.Contains(InChannelIndex);
	}

	const TArray<FOnset>& FOnsetNRTResult::GetOnsetsForChannel(int32 InChannelIndex) const
	{
		check(ChannelOnsets.Contains(InChannelIndex));

		return ChannelOnsets[InChannelIndex];
	}

	FFloatInterval FOnsetNRTResult::GetChannelOnsetInterval(int32 InChannelIndex) const
	{
		check(ChannelOnsets.Contains(InChannelIndex));

		return ChannelOnsetStrengthIntervals[InChannelIndex];
	}

	void FOnsetNRTResult::GetChannels(TArray<int32>& OutChannels) const
	{
		ChannelOnsets.GetKeys(OutChannels);
	}

	float FOnsetNRTResult::GetDurationInSeconds() const 
	{
		return DurationInSeconds;
	}

	void FOnsetNRTResult::SetDurationInSeconds(float InDuration)
	{
		DurationInSeconds = InDuration;
	}

	 /* Returns true if FOnsetNRTFrame arrays are sorted in chronologically ascending order via their timestamp.
	 */
	bool FOnsetNRTResult::IsSortedChronologically() const
	{
		return bIsSortedChronologically;
	}

	/**
	 * Sorts FOnsetNRTFrame arrays in chronologically ascnding order via their timestamp.
	 */
	void FOnsetNRTResult::SortChronologically()
	{
		for (auto& KeyValue : ChannelOnsets)
		{
			KeyValue.Value.Sort([](const FOnset& A, const FOnset& B) { return A.Timestamp < B.Timestamp; });
		}

		bIsSortedChronologically = true;
	}

	/*******************************************************************************/
	/*************************** FOnsetNRTWorker *******************************/
	/*******************************************************************************/

	FOnsetNRTWorker::FOnsetNRTWorker(const FAnalyzerNRTParameters& InParams, const FOnsetNRTSettings& InAnalyzerSettings)
	:	Settings(InAnalyzerSettings)
	,	NumFrames(0)
	,	NumChannels(InParams.NumChannels)
	,	SampleRate(InParams.SampleRate)
	,	NumWindowFrames(4096)
	,	NumWindowSamples(0)
	,	MonoScaling(1.f)
	{
		check(NumChannels > 0);
		check(SampleRate > 0.f);

		if (Settings.bDownmixToMono)
		{
			MonoBuffer.Reset(NumWindowFrames);
			MonoBuffer.AddUninitialized(NumWindowFrames);
			// equal power mixing
			MonoScaling = 1.f / FMath::Sqrt(static_cast<float>(NumChannels));

			// Only need one onset strength analyzer
			TUniquePtr<FOnsetStrengthAnalyzer> Analyzer = MakeUnique<FOnsetStrengthAnalyzer>(InAnalyzerSettings.OnsetStrengthSettings, SampleRate);
			OnsetStrengthAnalyzers.Emplace(MoveTemp(Analyzer));

			// Only need one onset strength array
			OnsetStrengths.AddDefaulted(1);
		}
		else
		{
			// Make individual onset strength analyzer for each channel
			for (int32 i = 0; i < NumChannels; i++)
			{
				TUniquePtr<FOnsetStrengthAnalyzer> Analyzer = MakeUnique<FOnsetStrengthAnalyzer>(InAnalyzerSettings.OnsetStrengthSettings, SampleRate);
				OnsetStrengthAnalyzers.Emplace(MoveTemp(Analyzer));
			}
			OnsetStrengths.AddDefaulted(NumChannels);
		}

		NumWindowSamples = NumChannels * NumWindowFrames;
		
		// Sliding buffer used for deinterleaving audio
		SlidingBuffer = MakeUnique<TSlidingBuffer<float> >(NumWindowSamples, NumWindowSamples);
	}


	void FOnsetNRTWorker::Analyze(TArrayView<const float> InAudio, IAnalyzerNRTResult* OutResult)
	{
		bool bDoFlush = false;

		AnalyzeMultichannel(InAudio, OutResult, bDoFlush);
	}

	void FOnsetNRTWorker::AnalyzeMultichannel(TArrayView<const float> InAudio, IAnalyzerNRTResult* OutResult, bool bDoFlush)
	{
		NumFrames += InAudio.Num() / NumChannels;

		// Assume that outer layers ensured that this is of correct type.
		FOnsetNRTResult* OnsetResult = static_cast<FOnsetNRTResult*>(OutResult);

		check(nullptr != OnsetResult);

		TAutoSlidingWindow<float, FAudioBufferAlignedAllocator> SlidingWindow(*SlidingBuffer, InAudio, HopBuffer, bDoFlush);

		if (Settings.bDownmixToMono)
		{
			for (const FAlignedFloatBuffer& Window : SlidingWindow)
			{
				FMemory::Memset(MonoBuffer.GetData(), 0, sizeof(float) * NumWindowFrames);

				// Sum all channels into mono buffer
				TAutoDeinterleaveView<float, FAudioBufferAlignedAllocator> DeinterleaveView(Window, ChannelBuffer, NumChannels);
				for(auto Channel : DeinterleaveView)
				{
					ArrayMixIn(Channel.Values, MonoBuffer);		
				}

				// Equal power sum. assuming incoherrent signals.
				ArrayMultiplyByConstantInPlace(MonoBuffer, 1.f / FMath::Sqrt(static_cast<float>(NumChannels)));

				AnalyzeWindow(MonoBuffer, 0, *OnsetResult);
			}
		}
		else
		{
			// Loop through entire array of input samples.
			for (const FAlignedFloatBuffer& Window : SlidingWindow)
			{
				// Each channel is analyzed seperately.
				for(const TDeinterleaveView<float>::TChannel<FAudioBufferAlignedAllocator> Channel : TAutoDeinterleaveView<float, FAudioBufferAlignedAllocator>(Window, ChannelBuffer, NumChannels))
				{
					AnalyzeWindow(Channel.Values, Channel.ChannelIndex, *OnsetResult);
				}
			}
		}
	}

	/** Called when analysis of audio asset is complete. */
	void FOnsetNRTWorker::Finalize(IAnalyzerNRTResult* OutResult) 
	{
		// Finish analyzing audio
		FAlignedFloatBuffer EmptyArray;
		bool bDoFlush = true;

		AnalyzeMultichannel(EmptyArray, OutResult, bDoFlush);

		// Call flush on all the onset strength analyzers
		for (int32 i = 0; i < OnsetStrengthAnalyzers.Num(); i++)
		{
			TArray<float> Strengths;
			OnsetStrengthAnalyzers[i]->FlushAudio(Strengths);
			OnsetStrengths[i].Append(Strengths);
		}

		// Assume that outer layers ensured that this is of correct type.
		FOnsetNRTResult* OnsetResult = static_cast<FOnsetNRTResult*>(OutResult);

		check(nullptr != OnsetResult);

		if (nullptr != OnsetResult)
		{
			ExtractOnsetsFromOnsetStrengths(*OnsetResult);

			float DurationInSeconds = static_cast<float>(NumFrames) / SampleRate;

			OnsetResult->SetDurationInSeconds(DurationInSeconds);

			OnsetResult->SortChronologically();
		}

		// Reset internal objects
		Reset();
	}

	void FOnsetNRTWorker::AnalyzeWindow(const FAlignedFloatBuffer& InWindow, int32 InChannelIndex, FOnsetNRTResult& OutResult)
	{
		TArray<float> Strengths;
		OnsetStrengthAnalyzers[InChannelIndex]->CalculateOnsetStrengths(InWindow, Strengths);

		OnsetStrengths[InChannelIndex].Append(Strengths);
	}

	void FOnsetNRTWorker::ExtractOnsetsFromOnsetStrengths(FOnsetNRTResult& OutResult)
	{
		for (int32 ChannelIndex = 0; ChannelIndex < OnsetStrengths.Num(); ChannelIndex++)
		{
			TArray<int32> OnsetIndices;
			TArray<float>& ChannelStrengths = OnsetStrengths[ChannelIndex];

			OnsetExtractIndices(Settings.PeakPickerSettings, ChannelStrengths, OnsetIndices);

			// Call this in case there are no onsets. Then the results object will know about the
			// channel, and will know there are no onsets. Otherwise it would be ambiguous whether 
			// the channel doesn't exist or if it does exist but contains no onsets.
			OutResult.AddChannel(ChannelIndex);

			for (int32 OnsetIndex : OnsetIndices)
			{
				float Timestamp = FOnsetStrengthAnalyzer::GetTimestampForIndex(Settings.OnsetStrengthSettings, SampleRate, OnsetIndex);
				OutResult.AddOnset(ChannelIndex, Timestamp, OnsetStrengths[ChannelIndex][OnsetIndex]);
			}
		}
	}

	void FOnsetNRTWorker::Reset()
	{
		// Reset internal counters
		NumFrames = 0;
		SlidingBuffer.Reset();

		for (int32 i = 0; i < OnsetStrengthAnalyzers.Num(); i++)
		{
			OnsetStrengthAnalyzers[i]->Reset();
		}
		for (int32 i = 0; i < OnsetStrengths.Num(); i++)
		{
			OnsetStrengths[i].Reset();
		}
	}

	// Name of this analyzer type.
	FName FOnsetNRTFactory::GetName() const
	{
		static FName FactoryName(TEXT("OnsetNRTFactory"));
		return FactoryName;
	}

	// Human readable name of this analyzer.
	FString FOnsetNRTFactory::GetTitle() const
	{
		return TEXT("Constant Q Analyzer Non-Real-Time");
	}

	// Create a new FOnsetNRTResult.
	TUniquePtr<IAnalyzerNRTResult> FOnsetNRTFactory::NewResult() const
	{
		return MakeUnique<FOnsetNRTResult>();
	}

	// Create a new FOnsetNRTWorker
	TUniquePtr<IAnalyzerNRTWorker> FOnsetNRTFactory::NewWorker(const FAnalyzerNRTParameters& InParams, const IAnalyzerNRTSettings* InSettings) const
	{
		const FOnsetNRTSettings* OnsetSettings = static_cast<const FOnsetNRTSettings*>(InSettings);

		check(nullptr != OnsetSettings);

		return MakeUnique<FOnsetNRTWorker>(InParams, *OnsetSettings);
	}
}

