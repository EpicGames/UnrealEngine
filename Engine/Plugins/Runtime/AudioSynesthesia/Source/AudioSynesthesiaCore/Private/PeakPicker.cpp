// Copyright Epic Games, Inc. All Rights Reserved.

#include "PeakPicker.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	FPeakPicker::FPeakPicker(const FPeakPickerSettings& InSettings)
	:	Settings(InSettings)
	{
		// Clamp settings to valid values
		Settings.NumPreMax = FMath::Max(1, Settings.NumPreMax);
		Settings.NumPostMax = FMath::Max(1, Settings.NumPostMax);
		Settings.NumPreMean = FMath::Max(1, Settings.NumPreMean);
		Settings.NumPostMean = FMath::Max(1, Settings.NumPostMean);
		Settings.NumWait = FMath::Max(1, Settings.NumWait);
		Settings.MeanDelta = FMath::Clamp(Settings.MeanDelta, 1e-6f, std::numeric_limits<float>::infinity());
	}

	void FPeakPicker::PickPeaks(TArrayView<const float> InData, TArray<int32>& OutPeakIndices)
	{
		OutPeakIndices.Empty();
		if (InData.Num() < 1)
		{
			return;
		}

		// Get max filtered version
		TArray<float> MaxFilteredData;
		int32 MaxFilterWindowSize = Settings.NumPreMax + Settings.NumPostMax + 1;
		int32 MaxFilterWindowOrigin = Settings.NumPreMax;

		ArrayMaxFilter(InData, MaxFilterWindowSize, MaxFilterWindowOrigin, MaxFilteredData);

		
		// Get mean filtered version
		TArray<float> MeanFilteredData;
		int32 MeanFilterWindowSize = Settings.NumPreMean + Settings.NumPostMean + 1;
		int32 MeanFilterWindowOrigin = Settings.NumPreMean;

		ArrayMeanFilter(InData, MeanFilterWindowSize, MeanFilterWindowOrigin, MeanFilteredData);

		// Find peak locations
		const float* InDataPtr = InData.GetData();
		const float* MaxDataPtr = MaxFilteredData.GetData();
		const float* MeanDataPtr = MeanFilteredData.GetData();

		const int32 Num = InData.Num();
		int32 PossiblePeakIndex = 0;
		while (PossiblePeakIndex < Num)
		{
			if (InDataPtr[PossiblePeakIndex] >= MaxDataPtr[PossiblePeakIndex])
			{
				if (InDataPtr[PossiblePeakIndex] >= (MeanDataPtr[PossiblePeakIndex] + Settings.MeanDelta))
				{
					OutPeakIndices.Add(PossiblePeakIndex);
					PossiblePeakIndex += Settings.NumWait;
					continue;
				}
			}
			PossiblePeakIndex++;
		}
	}
}

