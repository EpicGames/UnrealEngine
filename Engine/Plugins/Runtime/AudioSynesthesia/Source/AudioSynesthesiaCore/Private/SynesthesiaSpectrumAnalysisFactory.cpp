// Copyright Epic Games, Inc. All Rights Reserved.

#include "SynesthesiaSpectrumAnalysisFactory.h"
#include "DSP/DeinterleaveView.h"
#include "DSP/FloatArrayMath.h"
#include "DSP/SlidingWindow.h"

namespace Audio
{
	void FSynesthesiaSpectrumResult::Add(FSynesthesiaSpectrumEntry&& InEntry)
	{
		// Store Spectrum data in appropriate channel
		TArray<FSynesthesiaSpectrumEntry>& SpectrumArray = ChannelSpectrumArrays.FindOrAdd(InEntry.Channel);
		SpectrumArray.Add(MoveTemp(InEntry));
	}

	const TArray<FSynesthesiaSpectrumEntry>& FSynesthesiaSpectrumResult::GetChannelSpectrumArray(int32 ChannelIdx) const
	{
		return ChannelSpectrumArrays[ChannelIdx];
	}

	int32 FSynesthesiaSpectrumResult::GetNumChannels() const 
	{
		return ChannelSpectrumArrays.Num();
	}

	FSynesthesiaSpectrumAnalysisWorker::FSynesthesiaSpectrumAnalysisWorker(const FAnalyzerParameters& InParams, const FSynesthesiaSpectrumAnalysisSettings& InAnalyzerSettings)
		: NumChannels(InParams.NumChannels)
		, SampleRate(InParams.SampleRate)
	{
		check(NumChannels > 0);
		check(SampleRate > 0);
		check(FMath::IsPowerOfTwo(InAnalyzerSettings.FFTSize));
		
		// From IFFTAlgorithm::ForwardRealToComplex required number of output values 
		NumOutputFrames = InAnalyzerSettings.FFTSize / 2 + 1;
		NumWindowFrames = InAnalyzerSettings.FFTSize;
		NumWindowSamples = InAnalyzerSettings.FFTSize * InParams.NumChannels;

		NumHopFrames = FMath::CeilToInt(InAnalyzerSettings.AnalysisPeriod * (float)InParams.SampleRate);
		NumHopFrames = FMath::Max(1, NumHopFrames);

		int32 NumHopSamples = NumHopFrames * NumChannels;

		bDownmixToMono = InAnalyzerSettings.bDownmixToMono;
		if (bDownmixToMono)
		{
			MonoBuffer.Reset(NumWindowFrames);
			MonoBuffer.AddUninitialized(NumWindowFrames);
		}

		InternalBuffer = MakeUnique<TSlidingBuffer<float>>(NumWindowSamples, NumHopSamples);
		SpectrumAnalyzer = MakeUnique<FSynesthesiaSpectrumAnalyzer>(InParams.SampleRate, InAnalyzerSettings);
	}

	void FSynesthesiaSpectrumAnalysisWorker::Analyze(TArrayView<const float> InAudio, IAnalyzerResult* OutResult)
	{
		FSynesthesiaSpectrumResult* SpectrumResult = static_cast<FSynesthesiaSpectrumResult*>(OutResult);
		check(SpectrumResult != nullptr);

		TAutoSlidingWindow<float> SlidingWindow(*InternalBuffer, InAudio, InternalWindow);

		// Mono
		if (NumChannels == 1)
		{
			for (const TArray<float>& Window : SlidingWindow)
			{
				FSynesthesiaSpectrumEntry NewEntry;
				NewEntry.Channel = 0;
				NewEntry.Timestamp = static_cast<float>(FrameCounter) / SampleRate;
				NewEntry.SpectrumValues.AddZeroed(NumOutputFrames);

				SpectrumAnalyzer->ProcessAudio(Window, NewEntry.SpectrumValues);

				SpectrumResult->Add(MoveTemp(NewEntry));
				FrameCounter += NumHopFrames;
			}
			return;
		}

		// Multichannel
		if (bDownmixToMono)
		{
			for (const TArray<float>& Window : SlidingWindow)
			{
				FMemory::Memset(MonoBuffer.GetData(), 0, sizeof(float) * NumWindowFrames);

				// Downmix to mono
				for (int32 FrameIndex = 0; FrameIndex < MonoBuffer.Num(); ++FrameIndex)
				{
					for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
					{
						MonoBuffer[FrameIndex] += Window[FrameIndex * NumChannels + ChannelIndex];
					}
				}

				// Equal power sum. assuming incoherent signals.
				ArrayMultiplyByConstantInPlace(MonoBuffer, 1.f / FMath::Sqrt(static_cast<float>(NumChannels)));

				// Create results 
				FSynesthesiaSpectrumEntry NewEntry;
				NewEntry.Channel = 0; // Mono channel index
				NewEntry.Timestamp = static_cast<float>(FrameCounter) / SampleRate;
				NewEntry.SpectrumValues.AddZeroed(NumOutputFrames);

				SpectrumAnalyzer->ProcessAudio(MonoBuffer, NewEntry.SpectrumValues);

				SpectrumResult->Add(MoveTemp(NewEntry));
				FrameCounter += NumHopFrames;
			}
		}
		else // Calculate per channel 
		{
			for (const TArray<float>& Window : SlidingWindow)
			{
				int32 ChannelIndex = 0;
				TAutoDeinterleaveView<float, FAudioBufferAlignedAllocator> DeinterleaveView(Window, ChannelBuffer, NumChannels);
				for (auto Channel : DeinterleaveView)
				{
					// Create results 
					FSynesthesiaSpectrumEntry NewEntry;
					NewEntry.Channel = ChannelIndex;
					NewEntry.Timestamp = static_cast<float>(FrameCounter) / SampleRate;
					NewEntry.SpectrumValues.AddZeroed(NumOutputFrames);
					
					SpectrumAnalyzer->ProcessAudio(Channel.Values, NewEntry.SpectrumValues);

					SpectrumResult->Add(MoveTemp(NewEntry));
					ChannelIndex++;
				}
				FrameCounter += NumHopFrames;
			}
		}
	}

	FName FSynesthesiaSpectrumAnalysisFactory::GetName() const 
	{
		static FName FactoryName(TEXT("SpectrumAnalysisFactory"));
		return FactoryName;
	}

	FString FSynesthesiaSpectrumAnalysisFactory::GetTitle() const
	{
		return TEXT("Real-Time Spectrum Analyzer");
	}

	TUniquePtr<IAnalyzerResult> FSynesthesiaSpectrumAnalysisFactory::NewResult() const
	{
		TUniquePtr<FSynesthesiaSpectrumResult> Result = MakeUnique<FSynesthesiaSpectrumResult>();
		return Result;
	}

	TUniquePtr<IAnalyzerWorker> FSynesthesiaSpectrumAnalysisFactory::NewWorker(const FAnalyzerParameters& InParams, const IAnalyzerSettings* InSettings) const
	{
		const FSynesthesiaSpectrumAnalysisSettings* SpectrumSettings = static_cast<const FSynesthesiaSpectrumAnalysisSettings*>(InSettings);

		check(nullptr != SpectrumSettings);

		return MakeUnique<FSynesthesiaSpectrumAnalysisWorker>(InParams, *SpectrumSettings);
	}
}

