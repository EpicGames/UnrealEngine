// Copyright Epic Games, Inc. All Rights Reserved.

#include "SynesthesiaSpectrumAnalyzer.h"
#include "DSP/FloatArrayMath.h"

namespace Audio
{
	FSynesthesiaSpectrumAnalyzer::FSynesthesiaSpectrumAnalyzer(float InSampleRate, const FSynesthesiaSpectrumAnalyzerSettings& InSettings)
	: Settings(InSettings)
	, SampleRate(InSampleRate)
	, Window(InSettings.WindowType, InSettings.FFTSize, /*NumChannels=*/1, /*bIsPeriodic=*/false)
	{
		if (!ensure(SampleRate > 0.f))
		{
			SampleRate = 48000.f;
		}

		// Ranges from EFFTSize
		static const int32 MinLog2FFTSize = 6; 
		static const int32 MaxLog2FFTSize = 16; 

		// Create FFT
		FFFTSettings FFTSettings;

		int32 Log2FFTSize = MinLog2FFTSize; 
		while ((Settings.FFTSize > (1 << Log2FFTSize)) && (Log2FFTSize < MaxLog2FFTSize))
		{
			Log2FFTSize++;
		}

		FFTSettings.Log2Size = Log2FFTSize;
		FFTSettings.bArrays128BitAligned = true;
		FFTSettings.bEnableHardwareAcceleration = true;

		FFT = FFFTFactory::NewFFTAlgorithm(FFTSettings);
		check(FFT.IsValid());

		// Calculate forward scaling factor to apply to power spectrum
		switch (FFT->ForwardScaling())
		{
		case EFFTScaling::MultipliedByFFTSize:
			FFTScaling = 1.f / (Settings.FFTSize * Settings.FFTSize);
			break;

		case EFFTScaling::MultipliedBySqrtFFTSize:
			FFTScaling = 1.f / Settings.FFTSize;
			break;

		case EFFTScaling::DividedByFFTSize:
			FFTScaling = Settings.FFTSize * Settings.FFTSize;
			break;

		case EFFTScaling::DividedBySqrtFFTSize:
			FFTScaling = Settings.FFTSize;
			break;

		case EFFTScaling::None:
		default:
			FFTScaling = 1.f;
			break;
		}

		WindowedBuffer.SetNumUninitialized(FFT->NumInputFloats());
		FFTOutput.SetNumUninitialized(FFT->NumOutputFloats());
	}

	void FSynesthesiaSpectrumAnalyzer::ProcessAudio(TArrayView<const float> InSampleView, TArrayView<float> OutSpectrum)
	{
		if (!ensure(InSampleView.Num() == FFT->NumInputFloats()) || !ensure(OutSpectrum.Num() == FFT->NumOutputFloats() / 2))
		{
			return;
		}
		
		// Copy buffer and apply window
		FMemory::Memcpy(WindowedBuffer.GetData(), InSampleView.GetData(), sizeof(float) * Settings.FFTSize);
		Window.ApplyToBuffer(WindowedBuffer.GetData());

		// Perform FFT and convert to power 
		FFT->ForwardRealToComplex(WindowedBuffer.GetData(), FFTOutput.GetData());
		ArrayComplexToPower(FFTOutput, OutSpectrum);
		ArrayMultiplyByConstantInPlace(OutSpectrum, FFTScaling);

		switch (Settings.SpectrumType)
		{
			case ESynesthesiaSpectrumType::MagnitudeSpectrum:
				ArraySqrtInPlace(OutSpectrum);
				break;

			// Case already covered by complex to power above 
			case ESynesthesiaSpectrumType::PowerSpectrum:
				break;

			case ESynesthesiaSpectrumType::Decibel:
				ArrayPowerToDecibelInPlace(OutSpectrum, -90.f);
				break;

			default:
				checkf(false, TEXT("Unhandled ESynesthesiaSpectrumType"));
		}
	}

	const FSynesthesiaSpectrumAnalyzerSettings& FSynesthesiaSpectrumAnalyzer::GetSettings() const
	{
		return Settings;
	}
}
