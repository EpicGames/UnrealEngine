// Copyright Epic Games, Inc. All Rights Reserved.

#include "YINPitchDetector.h"

#include "DSP/BlockCorrelator.h"
#include "DSP/Dsp.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Logging/LogMacros.h"

DEFINE_LOG_CATEGORY_STATIC(LogYINPitchDetector, Log, All);

namespace Audio
{
	namespace YINPitchDetectorPrivate
	{
		constexpr int32 MinimumWindowSize = 512;
		constexpr int32 MaximumWindowSize = 8192;
	}

	FYINPitchDetector::FYINPitchDetector(const FYINPitchDetectorSettings& InSettings, float InSampleRate)
	:	Settings(InSettings)
	,	SampleRate(InSampleRate)
	,	WindowNumSamples(0)
	,	HalfWindowNumSamples(0)
	,	MinLagBin(1)
	,	MaxLagBin(1)
	,	WindowCounter(0)
	,	SlidingBuffer(1, 1)
	{
		using namespace YINPitchDetectorPrivate;

		if (!ensure(SampleRate > 0.f))
		{
			SampleRate = 1.f;
		}

		Settings.MinimumFrequency = FMath::Max(Settings.MinimumFrequency, 1.f);
		Settings.MaximumFrequency = FMath::Max(Settings.MaximumFrequency, Settings.MinimumFrequency);
		Settings.Threshold        = FMath::Clamp(Settings.Threshold, 0.f, 1.f);

		// Window must be at least 2x the longest period we expect to detect so that
		// the difference function has enough samples at the maximum lag.
		const int32 MinFreqNumSamples = FMath::RoundToInt(2.f * SampleRate / Settings.MinimumFrequency);

		int32 Log2WindowSize = 1;
		while ((1 << Log2WindowSize) < FMath::Max(MinFreqNumSamples, MinimumWindowSize))
		{
			Log2WindowSize++;
		}

		WindowNumSamples = 1 << Log2WindowSize;

		if (WindowNumSamples > MaximumWindowSize)
		{
			UE_LOGF(LogYINPitchDetector, Warning,
				"YIN: Pitches at minimum frequency (%.1f Hz) may not be detected reliably due to internal window size limit (%d samples)",
				Settings.MinimumFrequency, MaximumWindowSize);
			WindowNumSamples = MaximumWindowSize;
			Log2WindowSize   = FMath::FloorLog2(static_cast<uint32>(WindowNumSamples));
		}

		HalfWindowNumSamples = WindowNumSamples / 2;

		// Lag range corresponding to the requested frequency bounds.
		// τ = SampleRate / frequency  →  higher freq = shorter lag
		MinLagBin = FMath::Max(1, FMath::FloorToInt(SampleRate / Settings.MaximumFrequency));
		MaxLagBin = FMath::Min(HalfWindowNumSamples - 1, FMath::CeilToInt(SampleRate / Settings.MinimumFrequency));

		if (MinLagBin >= MaxLagBin)
		{
			UE_LOGF(LogYINPitchDetector, Warning,
				"YIN: Frequency range [%.1f, %.1f] Hz maps to a degenerate lag range [%d, %d]. Check settings.",
				Settings.MinimumFrequency, Settings.MaximumFrequency, MinLagBin, MaxLagBin);
		}

		const int32 HopNumSamples = FMath::Max(1, FMath::RoundToInt(SampleRate * Settings.AnalysisHopSeconds));

		SlidingBuffer = TSlidingBuffer<float>(WindowNumSamples, HopNumSamples);
		WindowBuffer.AddUninitialized(WindowNumSamples);

		// DiffBuffer covers lags 0 … HalfWindowNumSamples (inclusive).
		DiffBuffer.SetNumUninitialized(HalfWindowNumSamples + 1);

		// ---------------------------------------------------------------------------
		// FFT-based correlator setup.
		//
		// EWindowType::None  — rectangular window (all ones), so CrossCorrelate gives
		//                      the raw inner product with no amplitude distortion.
		// bDoNormalize=false — we apply our own CMND normalization in Step 2.
		// ---------------------------------------------------------------------------
		FBlockCorrelatorSettings CorrelatorSettings;
		CorrelatorSettings.Log2NumValuesInBlock = Log2WindowSize;
		CorrelatorSettings.WindowType           = EWindowType::None;
		CorrelatorSettings.bDoNormalize         = false;

		Correlator = MakeUnique<FBlockCorrelator>(CorrelatorSettings);

		if (Correlator.IsValid())
		{
			AutoCorrBuffer.AddUninitialized(Correlator->GetNumOutputValues());
		}

		// FirstHalfBuffer: first W/2 samples set per-frame, second half stays zero permanently.
		FirstHalfBuffer.AddZeroed(WindowNumSamples);
	}

	FYINPitchDetector::~FYINPitchDetector()
	{
	}

	// ---------------------------------------------------------------------------
	// Step 1 — FFT-based difference function
	//
	// The squared difference function expands as:
	//
	//   d(τ) = Σ_{t=0}^{W/2-1} (x[t] - x[t+τ])²
	//        = m₁ + m₂(τ) - 2r(τ)
	//
	// where:
	//   r(τ) = Σ_{t=0}^{W/2-1} x[t]·x[t+τ]        (cross-correlation, via FFT)
	//   m₁   = r(0) = Σ x[t]²        t ∈ [0, W/2)  (first-half energy, constant)
	//   m₂(τ) = Σ x[t]²              t ∈ [τ, τ+W/2) (shifted energy, O(1) sliding update)
	//
	// r(τ) is computed by cross-correlating the zero-padded first half of the
	// window against the full window using FBlockCorrelator (O(N log N) via FFT).
	// m₂(τ) is maintained with a single add/subtract per lag (O(1) per step).
	// ---------------------------------------------------------------------------
	void FYINPitchDetector::ComputeDifferenceFunction(const FAlignedFloatBuffer& InWindow)
	{
		if (!ensure(Correlator.IsValid()))
		{
			return;
		}

		const float* RESTRICT Data = InWindow.GetData();

		// Fill first half of FirstHalfBuffer from the analysis window.
		// The second half remains permanently zeroed (set in constructor).
		FMemory::Memcpy(FirstHalfBuffer.GetData(), Data, HalfWindowNumSamples * sizeof(float));

		// CrossCorrelate(A, B) internally computes IFFT(conj(FFT(B)) * FFT(A)),
		// giving Output[τ] = Σ_n B[n] * A[n+τ].
		// With B = FirstHalfBuffer (x[0..W/2-1] + zeros) and A = InWindow (x[0..W-1]):
		//   Output[τ] = Σ_{t=0}^{W/2-1} x[t] * x[t+τ]  = r(τ)  for τ ≤ W/2
		Correlator->CrossCorrelate(InWindow, FirstHalfBuffer, AutoCorrBuffer);

		const float* RESTRICT AutoCorr = AutoCorrBuffer.GetData();

		// m₁ = r(0) = Σ_{t=0}^{W/2-1} x[t]²
		const float M1 = AutoCorr[0];

		// d(0) is always 0 by definition.
		DiffBuffer[0] = 0.f;

		// m₂(0) = m₁ (same range, τ=0 means no shift)
		float M2 = M1;

		for (int32 Tau = 1; Tau <= HalfWindowNumSamples; ++Tau)
		{
			// Slide the W/2-length energy window one sample forward:
			//   remove x[τ-1]², add x[τ + W/2 - 1]²
			// Max index accessed: Tau + HalfWindowNumSamples - 1 = WindowNumSamples - 1 (in-bounds)
			M2 += Data[Tau + HalfWindowNumSamples - 1] * Data[Tau + HalfWindowNumSamples - 1]
			    - Data[Tau - 1] * Data[Tau - 1];

			// Clamp to non-negative: d(τ) is mathematically a sum of squares but
			// FFT rounding can produce small negative values that would distort CMND.
			DiffBuffer[Tau] = FMath::Max(0.f, M1 + M2 - 2.f * AutoCorr[Tau]);
		}
	}

	// ---------------------------------------------------------------------------
	// Step 2 — Cumulative mean normalized difference function (CMND)
	//
	//   d'(0) = 1
	//   d'(τ) = d(τ) / ( (1/τ) * Σ_{j=1}^{τ} d(j) )
	//
	// Normalizing by the running mean suppresses the spurious global minimum at
	// τ=0 and makes the threshold (Step 3) frequency-independent.
	// ---------------------------------------------------------------------------
	void FYINPitchDetector::NormalizeDifferenceFunction()
	{
		DiffBuffer[0] = 1.f;

		float RunningSum = 0.f;
		for (int32 Tau = 1; Tau <= HalfWindowNumSamples; ++Tau)
		{
			RunningSum += DiffBuffer[Tau];
			if (RunningSum > 0.f)
			{
				DiffBuffer[Tau] *= static_cast<float>(Tau) / RunningSum;
			}
			else
			{
				DiffBuffer[Tau] = 1.f;
			}
		}
	}

	// ---------------------------------------------------------------------------
	// Step 3 — Threshold + parabolic interpolation
	//
	// Find the first lag in [MinLagBin, MaxLagBin] where d'(τ) < Threshold.
	// If no such lag exists, fall back to the global minimum in that range.
	// Apply parabolic interpolation for sub-sample precision.
	// ---------------------------------------------------------------------------
	bool FYINPitchDetector::FindFundamentalLag(float& OutLag, float& OutStrength) const
	{
		// Guard against degenerate lag range from extreme settings (e.g., inverted
		// frequency bounds, window size clamp, or ensure fallback on SampleRate).
		// Without this check, the loops below would read past the end of DiffBuffer.
		if (MinLagBin >= DiffBuffer.Num() || MaxLagBin >= DiffBuffer.Num() || MinLagBin > MaxLagBin)
		{
			return false;
		}

		const float* DiffData = DiffBuffer.GetData();

		// --- threshold search (prefer smallest valid lag) ---
		int32 BestBin = INDEX_NONE;
		for (int32 Tau = MinLagBin; Tau <= MaxLagBin; ++Tau)
		{
			if (DiffData[Tau] < Settings.Threshold)
			{
				// Continue forward while the function keeps decreasing, to land on
				// the trough rather than its leading edge.
				while (Tau + 1 <= MaxLagBin && DiffData[Tau + 1] < DiffData[Tau])
				{
					++Tau;
				}
				BestBin = Tau;
				break;
			}
		}

		// --- fallback: global minimum in range ---
		if (BestBin == INDEX_NONE)
		{
			BestBin = MinLagBin;
			for (int32 Tau = MinLagBin + 1; Tau <= MaxLagBin; ++Tau)
			{
				if (DiffData[Tau] < DiffData[BestBin])
				{
					BestBin = Tau;
				}
			}

			// Still above threshold after global minimum search — unvoiced frame.
			if (DiffData[BestBin] >= Settings.Threshold)
			{
				return false;
			}
		}

		// --- parabolic interpolation for sub-sample lag ---
		// QuadraticPeakInterpolation expects the peak to be a maximum, but CMND
		// troughs are minima of the negated function, so we negate before calling.
		float PeakOffset = 0.f;
		float PeakValue  = 0.f;

		// Guard against edge bins where the three-point stencil would go out of bounds.
		if (BestBin > 0 && BestBin < HalfWindowNumSamples)
		{
			// Negate the three neighbours so the trough becomes a peak.
			const float Neighbours[3] = {
				-DiffData[BestBin - 1],
				-DiffData[BestBin],
				-DiffData[BestBin + 1]
			};

			if (QuadraticPeakInterpolation(Neighbours, PeakOffset, PeakValue))
			{
				// PeakValue is the negated CMND at the interpolated peak; un-negate.
				const float InterpolatedCMND = -PeakValue;
				OutLag      = static_cast<float>(BestBin) + PeakOffset;
				// Strength = 1 - CMND: higher periodicity → CMND near 0 → strength near 1.
				OutStrength = FMath::Clamp(1.f - InterpolatedCMND, 0.f, 1.f);
				return OutLag >= 1.f;
			}
		}

		// Interpolation failed or at boundary — use integer bin directly.
		OutLag      = static_cast<float>(BestBin);
		OutStrength = FMath::Clamp(1.f - DiffData[BestBin], 0.f, 1.f);
		return OutLag >= 1.f;
	}

	// ---------------------------------------------------------------------------
	// DetectPitches — main entry point
	// ---------------------------------------------------------------------------
	void FYINPitchDetector::DetectPitches(const FAlignedFloatBuffer& InMonoAudio, TArray<FPitchInfo>& OutPitches)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FYINPitchDetector::DetectPitches);
		TAutoSlidingWindow<float, FAudioBufferAlignedAllocator> SlidingWindow(SlidingBuffer, InMonoAudio, WindowBuffer);

		for (FAlignedFloatBuffer& AnalysisBuffer : SlidingWindow)
		{
			const int64 SampleCounter = (WindowCounter * static_cast<int64>(SlidingBuffer.GetNumHopSamples())) + SlidingBuffer.GetNumWindowSamples() / 2;
			const double Timestamp     = static_cast<double>(SampleCounter) / static_cast<double>(SampleRate);

			WindowCounter += 1;

			ComputeDifferenceFunction(AnalysisBuffer);
			NormalizeDifferenceFunction();

			float FundamentalLag = 0.f;
			float Strength       = 0.f;

			if (FindFundamentalLag(FundamentalLag, Strength))
			{
				FPitchInfo Info;
				Info.Frequency  = SampleRate / FundamentalLag;
				Info.Strength   = Strength;
				Info.Timestamp  = Timestamp;

				OutPitches.Add(MoveTemp(Info));
			}
		}
	}

	void FYINPitchDetector::Finalize(TArray<FPitchInfo>& OutPitches)
	{
		SlidingBuffer.Reset();
		WindowCounter = 0;
	}
}
