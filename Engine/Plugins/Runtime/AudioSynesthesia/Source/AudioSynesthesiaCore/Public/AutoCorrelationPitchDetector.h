// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PitchTracker.h"
#include "DSP/SlidingWindow.h"
#include "Templates/UniquePtr.h"

#define UE_API AUDIOSYNESTHESIACORE_API

namespace Audio { class FBlockCorrelator; }
namespace Audio { class FPeakPicker; }

namespace Audio
{
	/** Settings for creating an autocorrelation pitch detector. */
	struct FAutoCorrelationPitchDetectorSettings
	{
		/** Time (in seconds) between analysis windows. */
		float AnalysisHopSeconds = 0.01f;

		/** Minimum frequency of pitch result. */
		float MinimumFrequency = 20.f;

		/** Maximum frequency of pitch result. */
		float MaximumFrequency = 10000.f;

		/** Sensitivity of peak detection. Valid values are between 0.0 and 1.0. 
		 * 0.0 results in the least amount of pitches, while 1.0 result in the most pitches. */
		float Sensitivity = 0.5f;
	};

	/** Pitch detector based on autocorrelation. Note that autocorrelation pitch detectors 
	 * give more accurate frequency results for low frequencies, but have issues with octave errors.
	 * Generally, it will produce erroneous frequency observations in octaves _below_ the true
	 * pitch frequency.
	 */
	class FAutoCorrelationPitchDetector : public IPitchDetector
	{
		public:
			/** Create an auto correlation pitch detector with settings and a sample rate. */
			UE_API FAutoCorrelationPitchDetector(const FAutoCorrelationPitchDetectorSettings& InSettings, float InSampleRate);

			UE_API virtual ~FAutoCorrelationPitchDetector();

			/** Detect pitches in the audio. This can be called repeatedly with new audio.
			 *
			 * @param InMonoAudio - A mono audio buffer of any length.
			 * @param OutPitches - This array is filled with pitch observations.
			 */
			UE_API virtual void DetectPitches(const FAlignedFloatBuffer& InMonoAudio, TArray<FPitchInfo>& OutPitches) override;

			/** Resets internal audio buffers. This pitch detector does not produce any more pitches on Finalize. */
			UE_API virtual void Finalize(TArray<FPitchInfo>& OutPitches) override;

		private:

			FAutoCorrelationPitchDetectorSettings Settings;

			float SampleRate;
			int32 MinAutoCorrBin;
			int32 MaxAutoCorrBin;
			int32 WindowCounter;

			TSlidingBuffer<float> SlidingBuffer;
			FAlignedFloatBuffer WindowBuffer;
			FAlignedFloatBuffer AutoCorrBuffer;
			TUniquePtr<FBlockCorrelator> Correlator;

			TArray<int32> PeakIndices;
			TUniquePtr<FPeakPicker> PeakPicker;

	};
}

#undef UE_API
