// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/AudioFFT.h"
#include "DSP/ConstantQ.h"

#define UE_API AUDIOSYNESTHESIACORE_API

namespace Audio { class IFFTAlgorithm; }

namespace Audio
{
	/** Denotes which magnitude scaling to apply to output */
	enum class EConstantQScaling : uint8
	{
		/** CQT spectrum values conform to ESpectrumType, either magnitude or power. */
		Linear,

		/** CQT spectrum values are in decibels. */
		Decibel
	};

	/** FConstantQAnalyzerSettings
	 *
	 * 	Settings for the Constant Q Analyzer.
	 */
	struct FConstantQAnalyzerSettings : public FPseudoConstantQKernelSettings
	{
		/** Size of FFT describe in number of samples */
		int32 FFTSize;

		/** Type of window to be applied to input audio */
        EWindowType WindowType;

		/** Type of spectrum to use. */
		ESpectrumType SpectrumType;

		/** Type of scaling to use. */
		EConstantQScaling Scaling;

		FConstantQAnalyzerSettings()
		:	FFTSize(1024)
		,	WindowType(EWindowType::Blackman)
		,	SpectrumType(ESpectrumType::PowerSpectrum)
		,	Scaling(EConstantQScaling::Decibel)	
		{}
	};

	/** FConstantQAnalyzer
	 *
	 *  The Constant Q Analyzer computs a pseudo constant q transform of the input audio.
	 *  ConstantQ transforms produce frequency strength at logarithmically spaced frequency
	 *  intervals (as opposed to the linearly spaced intervals of the FFT). Logarithmically
	 *  spaced frequency intervals map nicely to musical scales.
	 */
	class FConstantQAnalyzer
	{
	public:

		/** Construct a FConstantQAnalyzer
		 *
		 * 	InSettings contains the parameters used to configure the analyzer.
		 * 	InSampleRate is the sample rate of the input audio.
		 */
		UE_API FConstantQAnalyzer(const FConstantQAnalyzerSettings& InSettings, const float InSampleRate);

		/** Calcualte Constant Q Transform of input audio.
		 *
		 * InSamples is an array of input audio. It must be of length GetSettings().FFTSize.
		 * OutCQT will be populated with the constants q transform values. It will have GetSettings().NumBands values.
		 */
		UE_API void CalculateCQT(const float* InSamples, TArray<float>& OutCQT);

		/** Returns a reference to the settings used in this analyzer */
		UE_API const FConstantQAnalyzerSettings& GetSettings() const;

	private:

		const FConstantQAnalyzerSettings Settings;

		const float SampleRate;

		int32 ActualFFTSize;

		/** Only the first (FFTSize / 2) + 1 FFT bins are used since FFT of real values is conjugate symmetric. */
		int32 NumUsefulFFTBins;

		/** Scale factor to normalize between different FFT Sizes */
		FWindow Window;

		FAlignedFloatBuffer WindowedSamples;
		FAlignedFloatBuffer ComplexSpectrum;
		FAlignedFloatBuffer RealSpectrum;

		TUniquePtr<IFFTAlgorithm> FFT;

		TUniquePtr<FContiguousSparse2DKernelTransform> CQTTransform;
	};
}

#undef UE_API
