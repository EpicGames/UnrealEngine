// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioAnalyzerInterface.h"
#include "ConstantQAnalyzer.h"
#include "ConstantQNRTFactory.h"

namespace Audio
{
	template <typename InSampleType> class TSlidingBuffer;

	/** FConstantQSettings
	 *
	 *  Settings for the Constant Q Non-Real-Time Factory.
	 */
	struct FConstantQSettings : public IAnalyzerSettings, public FConstantQAnalyzerSettings 
	{
	public:
		/** Time, in seconds, between constant Q frames. */
		float AnalysisPeriodInSeconds = 0.01f;

		/** 
		 * If true, all channels are mixed together with equal power before analysis. Only one channel
		 * is produced at channel index 0.
		 * If false, then each channel is analyzed separately. 
		 */
		bool bDownmixToMono = false;
	};

	/** 
	 * FLoudnessResult contains the temporal evolution of loudness for the last buffer of audio analyzed.
	 */
	class FConstantQResult : public IAnalyzerResult
	{
	public:

		FConstantQResult() {}

		/** Add a single frame of CQT data */
		AUDIOSYNESTHESIACORE_API void AddFrame(const int32 InChannelIndex, const float InTimestamp, TArrayView<const float> InSpectrum);

		/** Returns const reference to FLoudnessDatum array for individual channel. */
		AUDIOSYNESTHESIACORE_API const TArray<FConstantQFrame>& GetFramesForChannel(const int32 ChannelIdx) const;

		/** Returns the number of channels. */
		AUDIOSYNESTHESIACORE_API int32 GetNumChannels() const;

	private:
		TMap<int32, TArray<FConstantQFrame> > ChannelCQTFrames;
	};

	/** FConstantQWorker
	 *
	 *  FConstantQWorker computes a FConstantQResult from audio samples.
	 */
	class FConstantQWorker : public IAnalyzerWorker
	{
	public:

		/** Constructor
		 *
		 * InParams are the parameters which describe characteristics of the input audio.
		 * InAnalyzerSettings are the settings which control various aspects of the algorithm.
		 */
		AUDIOSYNESTHESIACORE_API FConstantQWorker(const FAnalyzerParameters& InParams, const FConstantQSettings& InAnalyzerSettings);

		/**
		 *  Analyze audio and put results into results pointer.
		 *
		 *  InAudio is an array view of audio.
		 *  OutResult is a pointer to a valid FConstantQResult
		 */
		AUDIOSYNESTHESIACORE_API virtual void Analyze(TArrayView<const float> InAudio, IAnalyzerResult* OutResult) override;

	private:
		/** Analyze audio with multiple channels interleaved. */
		void AnalyzeMultichannel(TArrayView<const float> InAudio, IAnalyzerResult* OutResult, const bool bDoFlush);

		/** Analyze a single window of audio from a single channel */
		void AnalyzeWindow(const FAlignedFloatBuffer& InWindow, const int32 InChannelIndex, FConstantQResult& OutResult);

		int32 NumChannels = 0;
		int32 NumBuffers = 0;
		float SampleRate = 0.0f;
		int32 NumHopFrames = 0;
		int32 NumHopSamples = 0;
		int32 NumWindowFrames = 0;
		int32 NumWindowSamples = 0;

		float MonoScaling = 1.0f;

		FAlignedFloatBuffer HopBuffer;
		FAlignedFloatBuffer ChannelBuffer;
		FAlignedFloatBuffer MonoBuffer;

		TArray<float> CQTSpectrum;

		TUniquePtr<TSlidingBuffer<float> > SlidingBuffer;
		TUniquePtr<FConstantQAnalyzer> ConstantQAnalyzer;

		bool bDownmixToMono = false;
	};

	/** FConstantQFactory
	 *  
	 *  Factory for creating FConstantQ workers and results
	 */
	class FConstantQFactory : public IAnalyzerFactory
	{
	public:

		/** Name of this analyzer type. */
		AUDIOSYNESTHESIACORE_API virtual FName GetName() const override;

		/** Human readable name of this analyzer. */
		AUDIOSYNESTHESIACORE_API virtual FString GetTitle() const override;

		/** Create a new FConstantQResult. */
		AUDIOSYNESTHESIACORE_API virtual TUniquePtr<IAnalyzerResult> NewResult() const override;

		/** Create a new FConstantQWorker 
		 *
		 *  InSettings must be a pointer to FConstantQSetting
		 */
		AUDIOSYNESTHESIACORE_API virtual TUniquePtr<IAnalyzerWorker> NewWorker(const FAnalyzerParameters& InParams, const IAnalyzerSettings* InSettings) const override;
	};
}