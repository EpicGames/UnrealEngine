// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioAnalyzerNRTInterface.h"
#include "ConstantQAnalyzer.h"

namespace Audio { template <typename InSampleType> class TSlidingBuffer; }

namespace Audio
{
	/** FConstantQNRTSettings
	 *
	 *  Settings for the Constant Q Non-Real-Time Factory.
	 */
	class FConstantQNRTSettings : public IAnalyzerNRTSettings, public FConstantQAnalyzerSettings 
	{
	public:

		/** Time, in seconds, between constant Q frames. */
		float AnalysisPeriod;

		/** 
		 * If true, all channels are mixed together with equal power before analysis. Only one channel
		 * is produced at channel index 0.
		 * If false, then each channel is analyzed separately. 
		 */
		bool bDownmixToMono;

		FConstantQNRTSettings()
		:	AnalysisPeriod(0.01f)
		,	bDownmixToMono(false)
		{}
	};


	/** FConstantQFrame
	 * 
	 *  Contains Constant Q data relating to one audio window.
	 */
	struct FConstantQFrame
	{
		/** Audio channel which produced the data. */
		int32 Channel;
		
		/** Timestamp in seconds referring to the center of the audio window. */
		float Timestamp;

		/** Output spectral data */
		TArray<float> Spectrum;

		FConstantQFrame()
		:	Channel(0)
		,	Timestamp(0.f)
		{}

		FConstantQFrame(int32 InChannelIndex, float InTimestamp, TArrayView<const float> InSpectrum)
		:	Channel(InChannelIndex)
		,	Timestamp(InTimestamp)
		,	Spectrum(InSpectrum.GetData(), InSpectrum.Num())
		{}
	};

	/** Serialize FConstantQFrame */
	AUDIOSYNESTHESIACORE_API FArchive &operator <<(FArchive& Ar, FConstantQFrame& Frame);

	/** FConstantQNRTResult
	 *
	 *  FConstantQNRTResult is a container for the output of the FConstantQNRTWorker.
	 */
	class FConstantQNRTResult : public IAnalyzerNRTResult
	{
	public:

		AUDIOSYNESTHESIACORE_API FConstantQNRTResult();

		/** Serialize or unserialize object */
		AUDIOSYNESTHESIACORE_API virtual void Serialize(FArchive& Archive) override;

		/** Add a single frame of CQT data */
		AUDIOSYNESTHESIACORE_API void AddFrame(int32 InChannelIndex, float InTimestamp, TArrayView<const float> InSpectrum);

		/** Returns true if this result contains data for the given channel index. */
		AUDIOSYNESTHESIACORE_API bool ContainsChannel(int32 InChannelIndex) const;

		/** Retrieve the array of frames for a single channel of audio. */
		AUDIOSYNESTHESIACORE_API const TArray<FConstantQFrame>& GetFramesForChannel(int32 InChannelIndex) const;

		/** Retrieve the difference between the maximum and minimum value in the spectrum. */
		AUDIOSYNESTHESIACORE_API FFloatInterval GetChannelConstantQInterval(int32 InChannelIdx) const;

		/** Retrieve an array of channel indices which exist in this result. */
		AUDIOSYNESTHESIACORE_API void GetChannels(TArray<int32>& OutChannels) const;

		/** Returns the duration of the analyzed audio in seconds */
		AUDIOSYNESTHESIACORE_API virtual float GetDurationInSeconds() const override;

		/** Sets the duration of the analyzed audio in seconds */
		AUDIOSYNESTHESIACORE_API void SetDurationInSeconds(float InDuration);

	 	/** Returns true if FConstantQFrame arrays are sorted in chronologically ascending order via their timestamp.  */
		AUDIOSYNESTHESIACORE_API bool IsSortedChronologically() const;

		/** Sorts FConstantQFrame arrays in chronologically ascnding order via their timestamp.  */
		AUDIOSYNESTHESIACORE_API void SortChronologically();


	private:

		float DurationInSeconds;

		TMap<int32, TArray<FConstantQFrame> > ChannelCQTFrames;

		TMap<int32, FFloatInterval> ChannelCQTIntervals;

		bool bIsSortedChronologically;
	};

	/** FConstantQNRTWorker
	 *
	 *  FConstantQNRTWorker computes a FConstantQNRTResult from audio samples.
	 */
	class FConstantQNRTWorker : public IAnalyzerNRTWorker
	{
	public:

		/** Constructor
		 *
		 * InParams are the parameters which describe characteristics of the input audio.
		 * InAnalyzerSettings are the settings which control various aspects of the algorithm.
		 */
		AUDIOSYNESTHESIACORE_API FConstantQNRTWorker(const FAnalyzerNRTParameters& InParams, const FConstantQNRTSettings& InAnalyzerSettings);

		/**
		 *  Analyze audio and put results into results pointer.
		 *
		 *  InAudio is an array view of audio.
		 *  OutResult is a pointer to a valid FConstantQNRTResult
		 */
		AUDIOSYNESTHESIACORE_API virtual void Analyze(TArrayView<const float> InAudio, IAnalyzerNRTResult* OutResult) override;

		/** 
		 *  Call when analysis of audio asset is complete. 
		 *
		 *  OutResult must be a pointer to a valid FConstantQNRTResult. 
		 */
		AUDIOSYNESTHESIACORE_API virtual void Finalize(IAnalyzerNRTResult* OutResult) override;

	private:

		/** Analyze audio with multiple channels interleaved. */
		void AnalyzeMultichannel(TArrayView<const float> InAudio, IAnalyzerNRTResult* OutResult, bool bDoFlush);

		/** Analyze a single window of audio from a single channel */
		void AnalyzeWindow(const FAlignedFloatBuffer& InWindow, int32 InChannelIndex, FConstantQNRTResult& OutResult);

		int32 NumFrames;
		int32 NumChannels;
		int32 NumBuffers;
		float SampleRate;
		int32 NumHopFrames;
		int32 NumHopSamples;
		int32 NumWindowFrames;
		int32 NumWindowSamples;

		float MonoScaling;

		FAlignedFloatBuffer HopBuffer;
		FAlignedFloatBuffer ChannelBuffer;
		FAlignedFloatBuffer MonoBuffer;

		TArray<float> CQTSpectrum;

		TUniquePtr<TSlidingBuffer<float> > SlidingBuffer;
		TUniquePtr<FConstantQAnalyzer> ConstantQAnalyzer;

		bool bDownmixToMono;
	};

	/** FConstantQNRTFactory
	 *  
	 *  Factory for creating FConstantQNRT workers and results
	 */
	class FConstantQNRTFactory : public IAnalyzerNRTFactory
	{
	public:

		/** Name of this analyzer type. */
		AUDIOSYNESTHESIACORE_API virtual FName GetName() const override;

		/** Human readable name of this analyzer. */
		AUDIOSYNESTHESIACORE_API virtual FString GetTitle() const override;

		/** Create a new FConstantQNRTResult. */
		AUDIOSYNESTHESIACORE_API virtual TUniquePtr<IAnalyzerNRTResult> NewResult() const override;

		/** Create a new FConstantQNRTWorker 
		 *
		 *  InSettings must be a pointer to FConstantQNRTSetting
		 */
		AUDIOSYNESTHESIACORE_API virtual TUniquePtr<IAnalyzerNRTWorker> NewWorker(const FAnalyzerNRTParameters& InParams, const IAnalyzerNRTSettings* InSettings) const override;
	};
}
