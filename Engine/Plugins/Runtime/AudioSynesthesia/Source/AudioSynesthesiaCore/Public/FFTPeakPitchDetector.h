// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PitchTracker.h"
#include "DSP/SlidingWindow.h"
#include "Templates/UniquePtr.h"

#define UE_API AUDIOSYNESTHESIACORE_API

namespace Audio { class FPeakPicker; }
namespace Audio { class IFFTAlgorithm; }

namespace Audio
{
	/** Settings for FFT base pitch detection */
	struct FFFTPeakPitchDetectorSettings
	{
		/** Log 2 size of the FFT */
		int32 Log2FFTSize = 13; // 2^13 = 8192

		/** Number of seconds represented in each analysis window. */
		float AnalysisWindowSeconds = 0.1f;

		/** Number of seconds between analysis windows. */
		float AnalysisHopSeconds = 0.01f;

		/** Minimum frequency of detected pitches. */
		float MinimumFrequency = 20.f;

		/** Maximum frequency of detected pitches. */
		float MaximumFrequency = 10000.f;

		/** Sensitivity of peak detection. Valid values are between 0.0 and 1.0. 
		 * 0.0 results in the least amount of pitches, while 1.0 result in the most pitches. */
		float Sensitivity = 0.5f;

		/** Reference SNR for calculating strength of signal. */
		float MaxStrengthSNRDecibels = 20.f;
	};

	/** FFT based pitch detector accel at detecting pitches at higher frequencies. */
	class FFFTPeakPitchDetector : public IPitchDetector
	{
		public:
			/** Create an fft pitch detector with settings and a sample rate. */
			UE_API FFFTPeakPitchDetector(const FFFTPeakPitchDetectorSettings& InSettings, float InSampleRate);

			UE_API virtual ~FFFTPeakPitchDetector();

			/** Detect pitches in the audio. This can be called repeatedly with new audio.
			 *
			 * @param InMonoAudio - A mono audio buffer of any length.
			 * @param OutPitches - This array is filled with pitch observations.
			 */
			UE_API virtual void DetectPitches(const FAlignedFloatBuffer& InMonoAudio, TArray<FPitchInfo>& OutPitches) override;

			/** Resets internal audio buffers. This pitch detector does not produce any more pitches on Finalize. */
			UE_API virtual void Finalize(TArray<FPitchInfo>& OutPitches) override;

		private:
			FFFTPeakPitchDetectorSettings Settings;

			float SampleRate;
			int32 FFTSize;
			int32 MinFFTBin;
			int32 MaxFFTBin;
			float FFTScaling;
			int32 WindowCounter;

			TUniquePtr<IFFTAlgorithm> FFTAlgorithm;
			TUniquePtr<FPeakPicker> PeakPicker;

			TSlidingBuffer<float> SlidingBuffer;
			FAlignedFloatBuffer WindowBuffer;
			FAlignedFloatBuffer ZeroPaddedAnalysisBuffer;
			FAlignedFloatBuffer ComplexSpectrumBuffer;
			FAlignedFloatBuffer PowerSpectrumBuffer;
	};
}

#undef UE_API
