// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/BinarySearch.h"
#include "Containers/ArrayView.h"
#include "Math/UnrealMath.h"


namespace Audio
{
	/** FindNearestByTimestamp
	 *
	 *  Returns a pointer to the element with the timestamp nearest to InTimestamp in absolute terms.. 
	 *
	 *  ElementType must be a type with a "float Timestamp;" member.
	 *  InArray is a array of ElementTypes which are sorted in ascending order via the Timestamp member.
	 *  InTimestamp is the query timestamp.
	 */
	template<typename ElementType>
	ElementType* FindNearestByTimestamp(TArrayView<ElementType> InArray, float InTimestamp)
	{
		if (0 == InArray.Num())
		{
			return nullptr;
		}
		
		// Get upper and lower array indices with FLoudnessData closest to the query timestamp.
		int32 UpperIndex = Algo::UpperBoundBy(InArray, InTimestamp, [](const ElementType& Datum) { return Datum.Timestamp; });
		int32 LowerIndex = UpperIndex - 1;

		// Ensure indices are within range.
		const int32 MaxIdx = InArray.Num() - 1;
		LowerIndex = FMath::Clamp(LowerIndex, 0, MaxIdx);
		UpperIndex = FMath::Clamp(UpperIndex, 0, MaxIdx);

		ElementType& LowerDatum = InArray[LowerIndex];
		ElementType& UpperDatum = InArray[UpperIndex];

		if (FMath::Abs(UpperDatum.Timestamp - InTimestamp) < FMath::Abs(LowerDatum.Timestamp - InTimestamp))
		{
			return &UpperDatum;
		}
		else
		{
			return &LowerDatum;
		}
	}
}


