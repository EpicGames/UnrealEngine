// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"

namespace Audio
{
	/** GetInterpolationParametersAtTimestamp
	 *
	 *  Calculates the indices and alpha parameter for interpolating between to elements in the array.
	 *
	 *  ElementType must be a type with a "float Timestamp;" member.
	 *  InArray is a array of ElementTypes which are sorted in ascending order via the Timestamp member.
	 *  InTimestamp is the query timestamp.
	 *
	 *  OutLowerIndex is populated with the index of the last element in InArray with a Timestamp less than or equal to InTimestamp. On error, this is set to INDEX_NONE
	 *  OutUpperIndex is populated with the index of the first element in InArray with a Timestamp greater than InTimestamp. On error, this is set to INDEX_NONE
	 *  OutAlpha is a number between 0.f and 1.f which describes the blending between the lower and upper indices. 0.f refers OutLowerIndex and 1.f to OutUpperIndex.
	 */
	template<typename ElementType>
	void GetInterpolationParametersAtTimestamp(TArrayView<const ElementType> InArray, float InTimestamp, int32& OutLowerIndex, int32& OutUpperIndex, float& OutAlpha)
	{
		if (0 == InArray.Num())
		{
			OutLowerIndex = INDEX_NONE;
			OutUpperIndex = INDEX_NONE;
			OutAlpha = 1.f;
			return;
		}
		
		// Get upper and lower array indices with FLoudnessData closest to the query timestamp.
		OutUpperIndex = Algo::UpperBoundBy(InArray, InTimestamp, [](const ElementType& Datum) { return Datum.Timestamp; });
		OutLowerIndex = OutUpperIndex - 1;

		// Ensure indices are within range.
		int32 MaxIdx = InArray.Num() - 1;
		OutLowerIndex = FMath::Clamp(OutLowerIndex, 0, MaxIdx);
		OutUpperIndex = FMath::Clamp(OutUpperIndex, 0, MaxIdx);

		const ElementType& LowerDatum = InArray[OutLowerIndex];
		const ElementType& UpperDatum = InArray[OutUpperIndex];

		// Determine time difference and linear interpolation factor from data.
		const float TimeDelta = FMath::Max(UpperDatum.Timestamp - LowerDatum.Timestamp, SMALL_NUMBER);
		OutAlpha = (InTimestamp - LowerDatum.Timestamp) / TimeDelta;
		OutAlpha = FMath::Clamp(OutAlpha, 0.0f, 1.0f);
	}
}

