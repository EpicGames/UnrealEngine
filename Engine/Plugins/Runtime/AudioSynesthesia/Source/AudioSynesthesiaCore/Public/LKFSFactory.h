// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/SortedMap.h"
#include "DSP/LKFSAnalyzer.h"
#include "IAudioAnalyzerInterface.h"

#include "LKFSFactory.generated.h"

namespace Audio { template <typename InSampleType> class TSlidingBuffer; }

/** Data representing the instantaneous loudness of a audio. */
USTRUCT(BlueprintType)
struct FLKFSResults
{
	GENERATED_BODY()

	/** The audio channel index which produced this result. If -1, then
	 * the data was produced by combining all other audio channels */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	int32 Channel = 0;

	/** Time in seconds of the source audio which corresponds to the loudness measurements. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float Timestamp = 0.f;

	/** The instantaneous, perceptually weighted energy relative to full scale. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float Energy = 0.f;

	/** The instantaneous, perceptually weighted loudness in dB. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float Loudness = 0.f;

	/** The average loudness over a short term window of time in dB. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float ShortTermLoudness = 0.f;

	/** The average loudness over a long term window of time in dB. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float IntegratedLoudness = 0.f;

	/** The average gated loudness over a long term window of time in dB. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float GatedLoudness = 0.f;

	/** Loudness Range (LRA), as defined in EBU Tech 3342. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	TOptional<FFloatInterval> LoudnessRange;
};


namespace Audio
{
	/**
	 * Contains settings for loudness analyzer.
	 */
	class FLKFSSettings : public IAnalyzerSettings, public FLKFSAnalyzerSettings
	{
	};


	/** FLKFSResult contains the temporal evolution of loudness. */
	class FLKFSAnalyzerResult : public IAnalyzerResult
	{
	public:
		/** Denotes the overall loudness channel index as opposed individual channel indices. */
		inline static const int32 ChannelIndexOverall = -1;

		FLKFSAnalyzerResult() = default;

		/** The number of individual channels available in the result. */
		AUDIOSYNESTHESIACORE_API int32 GetNumChannels() const;

		/** Add to the result. */
		AUDIOSYNESTHESIACORE_API void Add(FLKFSResults InResults);

		/** Returns const reference to FLKFSResults array for individual channel. */
		AUDIOSYNESTHESIACORE_API const TArray<FLKFSResults>& GetChannelLoudnessResults(int32 ChannelIdx) const;

		/** Returns const reference to FLKFSResults array associated with overall loudness. */
		AUDIOSYNESTHESIACORE_API const TArray<FLKFSResults>& GetLoudnessResults() const;

	private:
		TSortedMap<int32, TArray<FLKFSResults>> ChannelResults;
	};

	/** 
	 * FLKFSWorker performs loudness analysis on input sample buffers.
	 */
	class FLKFSWorker : public IAnalyzerWorker
	{
	public:
		/** Construct a worker */
		AUDIOSYNESTHESIACORE_API FLKFSWorker(const FAnalyzerParameters& InParams, const FLKFSSettings& InAnalyzerSettings);

		/**
		 * Analyzes input sample buffer and updates result. 
		 */
		AUDIOSYNESTHESIACORE_API virtual void Analyze(TArrayView<const float> InAudio, IAnalyzerResult* OutResult) override;

	private:
		int32 NumChannels = 0;
		FLKFSAnalyzer LKFSAnalyzer;
	};

	/**
	 * Defines the LKFS analyzer and creates related classes.
	 */
	class FLKFSFactory : public IAnalyzerFactory
	{
	public:

		/** Name of specific analyzer type. */
		AUDIOSYNESTHESIACORE_API virtual FName GetName() const override;

		/** Human readable name of analyzer. */
		AUDIOSYNESTHESIACORE_API virtual FString GetTitle() const override;

		/** Creates a new FLKFSAnalyzerResult */
		AUDIOSYNESTHESIACORE_API virtual TUniquePtr<IAnalyzerResult> NewResult() const override;

		/** Creates a new FLKFSWorker. This expects IAnalyzerSettings to be a valid pointer to a FLKFSSettings object. */
		AUDIOSYNESTHESIACORE_API virtual TUniquePtr<IAnalyzerWorker> NewWorker(const FAnalyzerParameters& InParams, const IAnalyzerSettings* InSettings) const override;
	};
}
