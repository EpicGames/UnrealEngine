// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/StridedView.h"
#include "DSP/LoudnessAnalyzer.h"
#include "DSP/SlidingWindow.h"
#include "IAudioAnalyzerNRTInterface.h"

#include "LKFSNRTFactory.generated.h"

namespace Audio { template <typename InSampleType> class TSlidingBuffer; }

/** Data representing the instantaneous loudness of a audio. */
USTRUCT(BlueprintType)
struct FLKFSNRTResults
{
	GENERATED_BODY()

	/** The audio channel index which produced this result. If -1, then
	 * the data was produced by combining all other audio channels */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	int32 Channel = 0;

	/** Time in seconds of the source audio which corresponds to the loudness measurements. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float Timestamp = 0.f;

	/** The instantaneous, perceptually weighted energy. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float Energy = 0.f;

	/** The instantaneous, perceptually weighted loudness. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float Loudness = 0.f;

	/** The average loudness over a short term window of time. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float ShortTermLoudness = 0.f;
};

	/** The loudness data associated with an entire file. */
USTRUCT(BlueprintType)
struct FLKFSNRTAggregateStats
{
	GENERATED_BODY()

	static constexpr float InvalidLoudness = TNumericLimits<float>::Min();
	
	/** The average loudness over the entire sound. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float IntegratedLoudness = InvalidLoudness;

	/** The gated loudness aggregated over the entire sound. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = AudioAnalyzer)
	float GatedLoudness = InvalidLoudness;
};

USTRUCT()
struct FLKFSNRTChannelData
{
	GENERATED_BODY()

	UPROPERTY()
	FLKFSNRTAggregateStats AggregateStats;

	UPROPERTY()
	TArray<FLKFSNRTResults> LoudnessArray;

	UPROPERTY()
	FFloatInterval LoudnessInterval;
};

USTRUCT()
struct FLKFSNRTWaveData
{
	GENERATED_BODY()

	UPROPERTY()
	bool bIsSortedChronologically = false;

	UPROPERTY()
	float DurationInSeconds = 0.f;

	UPROPERTY()
	TMap<int32, FLKFSNRTChannelData> ChannelData;
};

namespace Audio
{
	/**
	 * Contains settings for loudness analyzer.
	 */
	class FLKFSNRTSettings : public IAnalyzerNRTSettings 
	{
	public:
		static constexpr float DefaultShortTermLoudnessDuration = 3.f;

		FLoudnessAnalyzerSettings LoudnessAnalyzerSettings = { 
			.FFTSize = 4096,
			.WindowType = EWindowType::None,
			.LoudnessCurveType = ELoudnessCurveType::K,
			.MinAnalysisFrequency = 20.f,
			.MaxAnalysisFrequency = 24000.f,
			.ScalingMethod = ELoudnessAnalyzerScalingMethod::Corrected
		};

		/** Number of seconds between loudness measurements */
		float AnalysisPeriod = 0.1f;

		/** Number of seconds of audio analyzed for each loudness measurements */
		float AnalysisWindowDuration = 0.4f;

		/** Duration of audio analyzed for short term loudness. */
		float ShortTermLoudnessDuration = DefaultShortTermLoudnessDuration;
	};


	/** FLKFSNRTResult contains the temporal evolution of loudness. */
	class FLKFSNRTResult : public IAnalyzerNRTResult
	{
	public:
		/** Denotes the overall loudness channel index as opposed individual channel indices. */
		AUDIOSYNESTHESIACORE_API static const int32 ChannelIndexOverall;

		AUDIOSYNESTHESIACORE_API FLKFSNRTResult();

		/** Defines how to serialize result. */
		AUDIOSYNESTHESIACORE_API virtual void Serialize(FArchive& Archive) override;

		/** Initialize a channel if it does not already exist. */
		AUDIOSYNESTHESIACORE_API void AddChannel(int32 InChannelIndex);

		/** Appends an FLKFSNRTResults to the container. */
		AUDIOSYNESTHESIACORE_API void Add(const FLKFSNRTResults& InDatum);

		/** Returns true if this object data for the given channel index */
		AUDIOSYNESTHESIACORE_API bool ContainsChannel(int32 InChannelIndex) const;

		/** Returns const reference to FLKFSNRTResults array for individual channel. */
		AUDIOSYNESTHESIACORE_API TArrayView<const FLKFSNRTResults> GetChannelLoudnessArray(int32 ChannelIdx) const;

		/** Returns array view to FLKFSNRTResults array for individual channel. If
		 * a caller modifies the timestamps of data, they may need to resort the
		 * data by calling SortChronologically().
		 */
		AUDIOSYNESTHESIACORE_API TArrayView<FLKFSNRTResults> GetMutableChannelLoudnessArray(int32 ChannelIdx); 

		/** Returns const reference to FLKFSNRTResults array associated with overall loudness. */
		AUDIOSYNESTHESIACORE_API TArrayView<const FLKFSNRTResults> GetLoudnessArray() const;

		/** Return the aggregate stats associated with a specific channel */
		AUDIOSYNESTHESIACORE_API const FLKFSNRTAggregateStats* FindAggregateLoudnessStats(int32 InChannelIdx) const;

		/** Sets the aggregated loudness stats for a given channel */
		AUDIOSYNESTHESIACORE_API void SetAggregateLoudnessStats(int32 InChannelIdx, FLKFSNRTAggregateStats InStats);

		/** Returns the channel indices available in result. */
		AUDIOSYNESTHESIACORE_API void GetChannels(TArray<int32>& OutChannels) const;

		/** Gets the duration of the analyzed audio. */
		AUDIOSYNESTHESIACORE_API virtual float GetDurationInSeconds() const override;

		/** Sets the duration of the analyzed audio. */
		AUDIOSYNESTHESIACORE_API void SetDurationInSeconds(float InDuration);

		/** Returns true if FLKFSNRTResults arrays are sorted in chronologically ascending order via their timestamp. */
		AUDIOSYNESTHESIACORE_API bool IsSortedChronologically() const;

		/** Sorts FLKFSNRTResults arrays in chronologically ascending order via their timestamp. */
		AUDIOSYNESTHESIACORE_API void SortChronologically();

	private:
		FLKFSNRTWaveData Data;
	};

	/** 
	 * FLKFSNRTWorker performs loudness analysis on input sample buffers.
	 */
	class FLKFSNRTWorker : public IAnalyzerNRTWorker
	{
	public:
		/** Construct a worker */
		AUDIOSYNESTHESIACORE_API FLKFSNRTWorker(const FAnalyzerNRTParameters& InParams, const FLKFSNRTSettings& InAnalyzerSettings);

		/**
		 * Analyzes input sample buffer and updates result. 
		 */
		AUDIOSYNESTHESIACORE_API virtual void Analyze(TArrayView<const float> InAudio, IAnalyzerNRTResult* OutResult) override;

		/**
		 * Call when all audio data has been analyzed. 
		 */
		AUDIOSYNESTHESIACORE_API virtual void Finalize(IAnalyzerNRTResult* OutResult) override;

	private:


		void AnalyzeSlidingWindow(TAutoSlidingWindow<float>& InSlidingWindow, FLKFSNRTResult& OutResult);
		void AnalyzeWindow(TArrayView<const float> InWindow, FLKFSNRTResult& OutResult);

		void AddCurrentLoudnessDataToResult(FLKFSNRTResult& OutResult);
		void CalculateAggregateLoudnessStats(FLKFSNRTResult& InOutResult) const;
		void FinalizeChannels(FLKFSNRTResult& OutResult);

		int32 NumChannels = 0;
		int32 NumAnalyzedBuffers = 0;
		int32 NumSlidingWindowHopFrames = 0;
		int32 NumFrames = 0;
		float SampleRate = 0.f;
		float ShortTermLoudnessDuration = 0.f;
		int32 NumAnalysisHopWindows = 1;
		int32 NumAnalysisWindows = 1;
		TArray<float> ChannelPerceptualEnergy;
		TArray<float> OverallPerceptualEnergy;
		TArray<float> InternalWindow;
		TUniquePtr<TSlidingBuffer<float>> InternalBuffer;
		TUniquePtr<FMultichannelLoudnessAnalyzer> Analyzer;
	};

	/**
	 * Defines the LKFSNRT analyzer and creates related classes.
	 */
	class FLKFSNRTFactory : public IAnalyzerNRTFactory
	{
	public:

		/** Name of specific analyzer type. */
		AUDIOSYNESTHESIACORE_API virtual FName GetName() const override;

		/** Human readable name of analyzer. */
		AUDIOSYNESTHESIACORE_API virtual FString GetTitle() const override;

		/** Creates a new FLKFSNRTResult */
		AUDIOSYNESTHESIACORE_API virtual TUniquePtr<IAnalyzerNRTResult> NewResult() const override;

		/** Creates a new FLKFSNRTWorker. This expects IAnalyzerNRTSettings to be a valid pointer to a FLKFSNRTSettings object. */
		AUDIOSYNESTHESIACORE_API virtual TUniquePtr<IAnalyzerNRTWorker> NewWorker(const FAnalyzerNRTParameters& InParams, const IAnalyzerNRTSettings* InSettings) const override;
	};
}
