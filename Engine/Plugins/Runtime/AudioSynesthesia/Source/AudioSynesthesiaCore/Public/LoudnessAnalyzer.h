// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/CoreMiscDefines.h"
UE_DEPRECATED_HEADER(5.8, "Include DSP/LoudnessAnalyzer.h from the SignalProcessing module instead")

#include "DSP/LoudnessAnalyzer.h"
