// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/LoudnessAnalyzer.h"
#include "IAudioAnalyzerInterface.h"

namespace Audio { template <typename InSampleType> class TSlidingBuffer; }

namespace Audio
{
	/**
	 * Contains settings for loudness analyzer.
	 */
	class FLoudnessSettings : public IAnalyzerSettings, public FLoudnessAnalyzerSettings
	{
	public:
		/** Number of seconds between loudness measurements */
		float AnalysisPeriod = 0.01f;
	};

	/**
	 * Holds the loudness values per a time step.
	 */
	struct FLoudnessEntry
	{
		int32 Channel = 0;
		float Timestamp = 0.f;
		float Energy = 0.f;
		float Loudness = 0.f;
	};

	/** 
	 * FLoudnessResult contains the temporal evolution of loudness for the last buffer of audio analyzed.
	 */
	class FLoudnessResult : public IAnalyzerResult
	{
	public:
		/**
		 * Denotes the overall loudness as opposed individual channel indices.
		 */
		AUDIOSYNESTHESIACORE_API static const int32 ChannelIndexOverall;

		FLoudnessResult() {}

		/** Appends an FLoudnessDatum to the container. */
		AUDIOSYNESTHESIACORE_API void Add(const FLoudnessEntry& InDatum);

		/** Returns const reference to FLoudnessDatum array for individual channel. */
		AUDIOSYNESTHESIACORE_API const TArray<FLoudnessEntry>& GetChannelLoudnessArray(int32 ChannelIdx) const;

		/** Returns const reference to FLoudnessDatum array associated with overall loudness. */
		AUDIOSYNESTHESIACORE_API const TArray<FLoudnessEntry>& GetLoudnessArray() const;

		/** Returns the number of channels. */
		AUDIOSYNESTHESIACORE_API int32 GetNumChannels() const;

	private:
		TMap<int32, TArray<FLoudnessEntry> > ChannelLoudnessArrays;
	};

	/**
	 * FLoudnessWorker performs loudness analysis on input sample buffers.
	 */
	class FLoudnessWorker : public IAnalyzerWorker
	{
	public:
		/** Construct a worker */
		AUDIOSYNESTHESIACORE_API FLoudnessWorker(const FAnalyzerParameters& InParams, const FLoudnessSettings& InAnalyzerSettings);

		/** Analyzes input sample buffer and updates result. */
		AUDIOSYNESTHESIACORE_API virtual void Analyze(TArrayView<const float> InAudio, IAnalyzerResult* OutResult) override;

	private:

		/** Analyze a single window. */
		void AnalyzeWindow(TArrayView<const float> InWindow, FLoudnessResult& OutResult);

		int32 NumChannels = 0;
		int32 NumAnalyzedBuffers = 0;
		int32 NumHopFrames = 0;
		int32 SampleRate = 0;
		TArray<float> InternalWindow;
		TUniquePtr<TSlidingBuffer<float>> InternalBuffer;
		TUniquePtr<FMultichannelLoudnessAnalyzer> Analyzer;
	};

	/**
	 * Defines the Loudness analyzer and creates related classes.
	 */
	class FLoudnessFactory : public IAnalyzerFactory
	{
	public:

		/** Name of specific analyzer type. */
		AUDIOSYNESTHESIACORE_API virtual FName GetName() const override;

		/** Human readable name of analyzer. */
		AUDIOSYNESTHESIACORE_API virtual FString GetTitle() const override;

		/** Creates a new FLoudnessNRTResult */
		AUDIOSYNESTHESIACORE_API virtual TUniquePtr<IAnalyzerResult> NewResult() const override;

		/**  Creates a new FLoudnessWorker. This expects IAnalyzerNRTSettings to be a valid pointer to a FLoudnessSettings object. */
		AUDIOSYNESTHESIACORE_API virtual TUniquePtr<IAnalyzerWorker> NewWorker(const FAnalyzerParameters& InParams, const IAnalyzerSettings* InSettings) const override;
	};
}
