// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/LoudnessAnalyzer.h"
#include "IAudioAnalyzerNRTInterface.h"

namespace Audio { template <typename InSampleType> class TSlidingBuffer; }

namespace Audio
{
	/**
	 * Contains settings for loudness analyzer.
	 */
	class FLoudnessNRTSettings : public IAnalyzerNRTSettings, public FLoudnessAnalyzerSettings
	{
	public:
		/** Number of seconds between loudness measurements */
		float AnalysisPeriod;

		FLoudnessNRTSettings()
		:	AnalysisPeriod(0.01f)
		{}
	};


	/** 
	 * Holds the loudness values per a time step. 
	 */
	struct FLoudnessDatum
	{
		int32 Channel = 0;
		float Timestamp = 0.f;
		float Energy = 0.f;
		float Loudness = 0.f;
	};
	
	/** De/Serialize single loudness datum into archive. */
	AUDIOSYNESTHESIACORE_API FArchive &operator <<(FArchive& Ar, FLoudnessDatum& Datum);

	/** FLoudnessNRTResult contains the temporal evolution of loudness. */
	class FLoudnessNRTResult : public IAnalyzerNRTResult
	{
	public:
		/** Denotes the overall loudness channel index as opposed individual channel indices. */
		AUDIOSYNESTHESIACORE_API static const int32 ChannelIndexOverall;

		AUDIOSYNESTHESIACORE_API FLoudnessNRTResult();

		/** Defines how to serialize result. */
		AUDIOSYNESTHESIACORE_API virtual void Serialize(FArchive& Archive) override;

		/** Appends an FLoudnessDatum to the container. */
		AUDIOSYNESTHESIACORE_API void Add(const FLoudnessDatum& InDatum);

		/** Returns true if this object data for the given channel index */
		AUDIOSYNESTHESIACORE_API bool ContainsChannel(int32 InChannelIndex) const;

		/** Returns const reference to FLoudnessDatum array for individual channel. */
		AUDIOSYNESTHESIACORE_API const TArray<FLoudnessDatum>& GetChannelLoudnessArray(int32 ChannelIdx) const;

		/** Returns const reference to FLoudnessDatum array associated with overall loudness. */
		AUDIOSYNESTHESIACORE_API const TArray<FLoudnessDatum>& GetLoudnessArray() const;

		/** Returns range in dB of overall loudness result given the noise floor. */
		AUDIOSYNESTHESIACORE_API float GetLoudnessRange(float InNoiseFloor) const;

		/** Returns range in dB of loudness result given the noise floor. */
		AUDIOSYNESTHESIACORE_API float GetChannelLoudnessRange(int32 InChannelIdx, float InNoiseFloor) const;

		/** Returns the channel indices available in result. */
		AUDIOSYNESTHESIACORE_API void GetChannels(TArray<int32>& OutChannels) const;

		/** Gets the duration of the analyzed audio. */
		AUDIOSYNESTHESIACORE_API virtual float GetDurationInSeconds() const override;

		/** Sets the duration of the analyzed audio. */
		AUDIOSYNESTHESIACORE_API void SetDurationInSeconds(float InDuration);

		/** Returns true if FLoudnessDatum arrays are sorted in chronologically ascending order via their timestamp. */
		AUDIOSYNESTHESIACORE_API bool IsSortedChronologically() const;

		/** Sorts FLoudnessDatum arrays in chronologically ascending order via their timestamp. */
		AUDIOSYNESTHESIACORE_API void SortChronologically();

	private:
		float DurationInSeconds;
		TMap<int32, TArray<FLoudnessDatum> > ChannelLoudnessArrays;
		TMap<int32, FFloatInterval> ChannelLoudnessIntervals;
		bool bIsSortedChronologically;
	};

	/** 
	 * FLoudnessNRTWorker performs loudness analysis on input sample buffers.
	 */
	class FLoudnessNRTWorker : public IAnalyzerNRTWorker
	{
	public:
		/** Construct a worker */
		AUDIOSYNESTHESIACORE_API FLoudnessNRTWorker(const FAnalyzerNRTParameters& InParams, const FLoudnessNRTSettings& InAnalyzerSettings);

		/**
		 * Analyzes input sample buffer and updates result. 
		 */
		AUDIOSYNESTHESIACORE_API virtual void Analyze(TArrayView<const float> InAudio, IAnalyzerNRTResult* OutResult) override;

		/**
		 * Call when all audio data has been analyzed. 
		 */
		AUDIOSYNESTHESIACORE_API virtual void Finalize(IAnalyzerNRTResult* OutResult) override;

	private:

		// Analyze a single window
		void AnalyzeWindow(TArrayView<const float> InWindow, FLoudnessNRTResult& OutResult);

		int32 NumChannels;
		int32 NumAnalyzedBuffers;
		int32 NumHopFrames;
		int32 NumFrames;
		float SampleRate;
		TArray<float> InternalWindow;
		TUniquePtr<TSlidingBuffer<float>> InternalBuffer;
		TUniquePtr<FMultichannelLoudnessAnalyzer> Analyzer;
	};

	/**
	 * Defines the LoudnessNRT analyzer and creates related classes.
	 */
	class FLoudnessNRTFactory : public IAnalyzerNRTFactory
	{
	public:

		/** Name of specific analyzer type. */
		AUDIOSYNESTHESIACORE_API virtual FName GetName() const override;

		/** Human readable name of analyzer. */
		AUDIOSYNESTHESIACORE_API virtual FString GetTitle() const override;

		/** Creates a new FLoudnessNRTResult */
		AUDIOSYNESTHESIACORE_API virtual TUniquePtr<IAnalyzerNRTResult> NewResult() const override;

		/** Creates a new FLoudnessNRTWorker. This expects IAnalyzerNRTSettings to be a valid pointer to a FLoudnessNRTSettings object. */
		AUDIOSYNESTHESIACORE_API virtual TUniquePtr<IAnalyzerNRTWorker> NewWorker(const FAnalyzerNRTParameters& InParams, const IAnalyzerNRTSettings* InSettings) const override;
	};
}
