// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PitchTracker.h"
#include "Templates/UniquePtr.h"

#define UE_API AUDIOSYNESTHESIACORE_API

namespace Audio
{
	/** Settings for FMaxStrengthPitchTracker */
	struct FMaxStrengthPitchTrackerSettings
	{
		/** Minimum strength of pitch to be included in pitch track. */
		float MinimumStrength = 0.1f;

		/** Maximum deviation of ratio between two freuqencies to consider them part of the same pitch track. */
		float MaximumFrequencyRatioDeviation = 0.05f;
	};

	/** Track the maximum strength pitch. */
	class FMaxStrengthPitchTracker : public IPitchTracker
	{
		public:
			/** Constructor accepts settings and a unique pointer to the pitch detector. */
			UE_API FMaxStrengthPitchTracker(const FMaxStrengthPitchTrackerSettings& InSettings, TUniquePtr<IPitchDetector> InDetector);

			UE_API virtual ~FMaxStrengthPitchTracker();

			/** Append pitch tracks to processed audio. */
			UE_API virtual void TrackPitches(const FAlignedFloatBuffer& InMonoAudio, TArray<FPitchTrackInfo>& OutPitchTracks) override;

			/** Call when all audio has been processed. Appends pitch tracks to OutPitchTracks. */
			UE_API virtual void Finalize(TArray<FPitchTrackInfo>& OutPitchTracks) override;

		private:
			FMaxStrengthPitchTrackerSettings Settings;
			TUniquePtr<IPitchDetector> PitchDetector;

			TArray<FPitchInfo> Observations;
	};
}

#undef UE_API
