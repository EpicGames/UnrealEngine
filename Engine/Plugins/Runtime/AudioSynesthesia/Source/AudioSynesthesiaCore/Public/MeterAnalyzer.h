// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/EnvelopeFollower.h"

#define UE_API AUDIOSYNESTHESIACORE_API

namespace Audio
{
	struct FMeterAnalyzerSettings
	{	
		// Envelope follower mode
		EPeakMode::Type MeterPeakMode = EPeakMode::RootMeanSquared;

		// Envelope follower attack time in milliseconds
		int32 MeterAttackTime = 300;

		// Envelope follower release time in milliseconds
		int32 MeterReleaseTime = 300;
		
		// Peak detector hold time in milliseconds
		int32 PeakHoldTime = 100;

		// The volume threshold to detect clipping
		float ClippingThreshold = 1.0f;

		bool bIsAnalog = true;
	};

	// Per-channel analyzer results
	struct FMeterAnalyzerResults
	{
		float TimeSec = 0.0f;
		TArray<float> MeterValues;
		TArray<float> PeakValues;
		TArray<float> ClippingValues;
		TArray<int32> NumSamplesClipping;
	};

	class FMeterAnalyzer
	{
	public:

		/** Construct analyzer */
		UE_API FMeterAnalyzer(float InSampleRate, int32 NumChannels, const FMeterAnalyzerSettings& InSettings);

		/**
		 * Calculate the meter results for the input samples.  Will return the current meter and the current peak value of the analyzer.
		 */
		UE_API FMeterAnalyzerResults ProcessAudio(TArrayView<const float> InSampleView);

		/**
		 * Return const reference to settings used inside this analyzer.
		 */
		UE_API const FMeterAnalyzerSettings& GetSettings() const;

	protected:
		// Envelope follower per channel
		FEnvelopeFollower EnvelopeFollower;
		FAlignedFloatBuffer EnvelopeBuffer;

		// Per-channel clipping data
		struct FClippingData
		{
			// How many samples we've been clipping
			int32 NumSamplesClipping = 0;
			float ClippingValue = 0.0f;
		};
		TArray<FClippingData> ClippingDataPerChannel;

		// State to track the peak data
		struct FEnvelopeData
		{
			float MaxEnvelopeValue = 0.f;
			float PeakValue = 0.0f;
			int32 FramesUntilPeakReset = 0;

			// Current slope of the envelope
			float PriorEnvelopeValue = 0.f;
			bool bPriorEnvelopeSlopeIsPositive = false;
		};
		// Per-channel peak data
		TArray<FEnvelopeData> EnvelopeDataPerChannel;


		FMeterAnalyzerSettings Settings;
		float SampleRate = 0.0f;
		int32 NumChannels = 0;
		int32 PeakHoldFrames = 0;
		int64 FrameCounter = 0;
	};
}

#undef UE_API
