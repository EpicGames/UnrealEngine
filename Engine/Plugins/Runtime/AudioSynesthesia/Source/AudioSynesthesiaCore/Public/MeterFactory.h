// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeterAnalyzer.h"
#include "IAudioAnalyzerInterface.h"

namespace Audio { template <typename InSampleType> class TSlidingBuffer; }

namespace Audio
{
	/**
	 * Contains settings for meter  analyzer.
	 */
	class FMeterSettings : public IAnalyzerSettings, public FMeterAnalyzerSettings
	{	
	public:
		/** Number of seconds between loudness measurements */
		float AnalysisPeriod = 0.01f;
	};

	/**
	 * Holds the meter results per a time step for each channel
	 */
	struct FMeterEntry
	{
		int32 Channel = 0;
		float Timestamp = 0.0f;
		float MeterValue = 0.0f;
		float PeakValue = 0.0f;
		float ClippingValue = 0.0f;
		int32 NumSamplesClipping = 0;
	};

	/** 
	 * FMeterResult contains the temporal evolution of loudness for the last buffer of audio analyzed.
	 */
	class FMeterResult : public IAnalyzerResult
	{
	public:
		/**
		 * Denotes the overall loudness as opposed individual channel indices.
		 */
		AUDIOSYNESTHESIACORE_API static const int32 ChannelIndexOverall;

		FMeterResult() {}

		/** Appends an FMeterEntry to the container. */
		AUDIOSYNESTHESIACORE_API void Add(const FMeterEntry& InEntry);

		/** Returns const reference to FMeterEntry array for individual channel. */
		AUDIOSYNESTHESIACORE_API const TArray<FMeterEntry>& GetChannelMeterArray(int32 ChannelIdx) const;

		/** Returns const reference to FMeterEntry array associated with overall loudness. */
		AUDIOSYNESTHESIACORE_API const TArray<FMeterEntry>& GetMeterArray() const;

		/** Returns the number of channels. */
		AUDIOSYNESTHESIACORE_API int32 GetNumChannels() const;

	private:
		float DurationInSeconds = 0.0f;
		TMap<int32, TArray<FMeterEntry> > ChannelMeterArrays;
	};

	/**
	 * FMeterWorker performs meter analysis on input sample buffers.
	 */
	class FMeterWorker : public IAnalyzerWorker
	{
	public:
		/** Construct a worker */
		AUDIOSYNESTHESIACORE_API FMeterWorker(const FAnalyzerParameters& InParams, const FMeterSettings& InAnalyzerSettings);

		/** Analyzes input sample buffer and updates result. */
		AUDIOSYNESTHESIACORE_API virtual void Analyze(TArrayView<const float> InAudio, IAnalyzerResult* OutResult) override;

	private:

		int32 NumChannels = 0;
		int32 NumAnalyzedBuffers = 0;
		int32 NumHopFrames = 0;
		int32 SampleRate = 0;
		TArray<float> InternalWindow;
		TUniquePtr<TSlidingBuffer<float>> InternalBuffer;
		TUniquePtr<FMeterAnalyzer> MeterAnalyzer;
	};

	/**
	 * Defines the meter analyzer and creates related classes.
	 */
	class FMeterFactory : public IAnalyzerFactory
	{
	public:
		/** Name of specific analyzer type. */
		AUDIOSYNESTHESIACORE_API virtual FName GetName() const override;

		/** Human readable name of analyzer. */
		AUDIOSYNESTHESIACORE_API virtual FString GetTitle() const override;

		/** Creates a new FLoudnessNRTResult */
		AUDIOSYNESTHESIACORE_API virtual TUniquePtr<IAnalyzerResult> NewResult() const override;

		/**  Creates a new FLoudnessWorker. This expects IAnalyzerNRTSettings to be a valid pointer to a FLoudnessSettings object. */
		AUDIOSYNESTHESIACORE_API virtual TUniquePtr<IAnalyzerWorker> NewWorker(const FAnalyzerParameters& InParams, const IAnalyzerSettings* InSettings) const override;
	};
}
