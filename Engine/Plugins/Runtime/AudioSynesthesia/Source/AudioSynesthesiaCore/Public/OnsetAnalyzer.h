// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/AudioFFT.h"
#include "DSP/MelScale.h"
#include "DSP/SlidingWindow.h"

#define UE_API AUDIOSYNESTHESIACORE_API

namespace Audio { class IFFTAlgorithm; }
namespace Audio { struct FPeakPickerSettings; }

namespace Audio
{
	/** Onset strength settings */
	struct FOnsetStrengthSettings
	{
		// Number of frames between strength windows
		int32 NumHopFrames = 1024;

		// Number of frames used to represent one audio window
		int32 NumWindowFrames = 4096;

		// Number of windows to lag when calculating window difference.
		int32 ComparisonLag = 4;

		// Size of FFT. Must be greater than or equal to NumWindowFrames.
		int32 FFTSize = 4096;

		// Type of analysis window to apply to audio.
		EWindowType WindowType = EWindowType::Blackman;

		// Decibel level considered to be silence.
		float NoiseFloorDb = -60.f;

		// Settings for converting FFT to mel spectrum.
		FMelSpectrumKernelSettings MelSettings;
	};

	/** FOnsetStrengthAnalyzer
	 *
	 * FOnsetStrengthAnalyzer calculates the onset strength from audio. 
	 *
	 * Onset strength is calculated as the half wave rectificed difference between two spectral frames.
	 * This onset strength onset analyzer uses the following approach.
	 *
	 * [audio]->[fft]->[mel spectrum]->[diff]->[half wave rectify]->[mean]->[onset strength]
	 *                       |           |
	 *                       >-[lag]-----|
	 */
	class FOnsetStrengthAnalyzer
	{
		public:
			UE_API FOnsetStrengthAnalyzer(const FOnsetStrengthSettings& InSettings, float InSampleRate);

			/** Calculates onset strengths from audio and fills OutOnsetStrengths with generated onset strengths */
			UE_API void CalculateOnsetStrengths(TArrayView<const float> InSamples, TArray<float>& OutOnsetStrengths);

			/** Call when done processing audio for an audio analyzer */
			UE_API void FlushAudio(TArray<float>& OutEnvelopeStrengths);
			
			/** Call to reset internal counters and lag spectra. */
			UE_API void Reset();

			/** Converts an onset strength index into a timestamp. */
			static UE_API float GetTimestampForIndex(const FOnsetStrengthSettings& InSettings, float InSampleRate, int32 InIndex);

		private:

			void AnalyzeAudio(TArrayView<const float> InSamples, TArray<float>& OutEnvelopeStrengths, bool bDoFlush = false);

			// Processes single audio window
			float GetNextOnsetStrength(TArrayView<const float> InSamples);

			FOnsetStrengthSettings Settings;
			int32 LagSpectraIndex;
			int32 ActualFFTSize;

			TSlidingBuffer<float> SlidingBuffer;
			FAlignedFloatBuffer WorkingBuffer;
			FAlignedFloatBuffer WindowedSamples;
			FAlignedFloatBuffer ComplexSpectrum;
			FAlignedFloatBuffer RealSpectrum;

			// Array of arrays to hold lag spectra.
			TArray<FAlignedFloatBuffer> PreviousMelSpectra;

			TArray<float> MelSpectrum;
			TArray<float> MelSpectrumDifference;

			FWindow Window;
			TUniquePtr<FContiguousSparse2DKernelTransform> MelTransform;
			TUniquePtr<IFFTAlgorithm> FFT;
	};

	/** 
	 * Extract onset indicies from an onset strength envelope.
	 *
	 * InSettings is the peak picker settings used to pick onset peaks from an onset strength envelope.
	 * InOnsetEnvelop is an onset strength envelope generated using the FOnsetStrengthAnalyzer.
	 * OutOnsetIndices contains the indices where onsets occur.
	 */
	AUDIOSYNESTHESIACORE_API void OnsetExtractIndices(const FPeakPickerSettings& InSettings, TArrayView<const float> InOnsetEnvelope, TArray<int32>& OutOnsetIndices);

	/**
	 * Backtracks onset indices to the beginning of the onset attack. This is useful when generating 
	 * splice points which retain the onset attack. 
	 */
	AUDIOSYNESTHESIACORE_API void OnsetBacktrackIndices(TArrayView<const float> InOnsetEnvelope, TArrayView<const int32> InOnsetIndices, TArray<int32>& OutBacktrackedOnsetIndices);
}

#undef UE_API
