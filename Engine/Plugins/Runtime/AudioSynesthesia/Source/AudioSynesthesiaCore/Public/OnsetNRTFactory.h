// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioAnalyzerNRTInterface.h"
#include "OnsetAnalyzer.h"
#include "PeakPicker.h"

#define UE_API AUDIOSYNESTHESIACORE_API

namespace Audio { template <typename InSampleType> class TSlidingBuffer; }

namespace Audio
{
	/** FOnsetNRTSettings
	 *
	 *  Settings for the Constant Q Non-Real-Time Factory.
	 */
	class FOnsetNRTSettings : public IAnalyzerNRTSettings
	{
	public:

		FOnsetStrengthSettings OnsetStrengthSettings;

		FPeakPickerSettings PeakPickerSettings;

		/** 
		 * If true, all channels are mixed together with equal power before analysis. Only one channel
		 * is produced at channel index 0.
		 * If false, then each channel is analyzed separately. 
		 */
		bool bDownmixToMono;

		FOnsetNRTSettings()
		:	bDownmixToMono(false)
		{}
	};


	/** FOnset
	 * 
	 *  Contains data relating to one onset.
	 */
	struct FOnset
	{
		/** Audio channel which produced the data. */
		int32 Channel;
		
		/** Timestamp in seconds referring to the center of the audio window. */
		float Timestamp;

		float Strength;

		FOnset()
		:	Channel(0)
		,	Timestamp(0.f)
		,	Strength(0.f)
		{}

		FOnset(int32 InChannelIndex, float InTimestamp, float InStrength)
		:	Channel(InChannelIndex)
		,	Timestamp(InTimestamp)
		,	Strength(InStrength)
		{}
	};

	/** Serialize FOnset */
	AUDIOSYNESTHESIACORE_API FArchive &operator <<(FArchive& Ar, FOnset& Frame);

	/** FOnsetNRTResult
	 *
	 *  FOnsetNRTResult is a container for the output of the FOnsetNRTWorker.
	 */
	class FOnsetNRTResult : public IAnalyzerNRTResult
	{
	public:

		UE_API FOnsetNRTResult();

		/** Serialize or unserialize object */
		UE_API virtual void Serialize(FArchive& Archive) override;

		/** Add a single frame of CQT data */
		UE_API void AddOnset(int32 InChannelIndex, float InTimestamp, float InStrength);

		/** Adds an empty channel. Useful if a channel exists without any onsets. */
		UE_API void AddChannel(int32 InChannelIndex);

		/** Returns true if this result contains the channel index. */
		UE_API bool ContainsChannel(int32 InChannelIndex) const;

		/** Retrieve the array of frames for a single channel of audio. */
		UE_API const TArray<FOnset>& GetOnsetsForChannel(int32 InChannelIndex) const;

		/** Retrieve the interval of the maximum and minimum strengths of all onsets in channel. */
		UE_API FFloatInterval GetChannelOnsetInterval(int32 InChannelIdx) const;

		/** Retrieve the channels available in the result. */
		UE_API void GetChannels(TArray<int32>& OutChannels) const;

		/** Returns the duration of the analyzed audio in seconds */
		UE_API virtual float GetDurationInSeconds() const override;

		/** Sets the duration of the analyzed audio in seconds */
		UE_API void SetDurationInSeconds(float InDuration); 

	 	/** Returns true if FOnset arrays are sorted in chronologically ascending order via their timestamp.  */
		UE_API bool IsSortedChronologically() const;

		/** Sorts FOnset arrays in chronologically ascnding order via their timestamp.  */
		UE_API void SortChronologically();

	private:

		float DurationInSeconds;

		TMap<int32, TArray<FOnset> > ChannelOnsets;

		TMap<int32, FFloatInterval> ChannelOnsetStrengthIntervals;

		bool bIsSortedChronologically;
	};

	/** FOnsetNRTWorker
	 *
	 *  FOnsetNRTWorker computes a FOnsetNRTResult from audio samples.
	 */
	class FOnsetNRTWorker : public IAnalyzerNRTWorker
	{
	public:

		/** Constructor
		 *
		 * InParams are the parameters which describe characteristics of the input audio.
		 * InAnalyzerSettings are the settings which control various aspects of the algorithm.
		 */
		UE_API FOnsetNRTWorker(const FAnalyzerNRTParameters& InParams, const FOnsetNRTSettings& InAnalyzerSettings);

		/**
		 *  Analyze audio and put results into results pointer.
		 *
		 *  InAudio is an array view of audio.
		 *  OutResult is a pointer to a valid FOnsetNRTResult
		 */
		UE_API virtual void Analyze(TArrayView<const float> InAudio, IAnalyzerNRTResult* OutResult) override;

		/** 
		 *  Call when analysis of audio asset is complete. 
		 *
		 *  OutResult must be a pointer to a valid FOnsetNRTResult. 
		 */
		UE_API virtual void Finalize(IAnalyzerNRTResult* OutResult) override;

	private:

		/** Analyze audio with multiple channels interleaved. */
		void AnalyzeMultichannel(TArrayView<const float> InAudio, IAnalyzerNRTResult* OutResult, bool bDoFlush);

		void AnalyzeWindow(const FAlignedFloatBuffer& InWindow, int32 InChannelIndex, FOnsetNRTResult& OutResult);

		void ExtractOnsetsFromOnsetStrengths(FOnsetNRTResult& OutResult);

		void Reset();

		FOnsetNRTSettings Settings;

		int32 NumFrames;
		int32 NumChannels;
		float SampleRate;
		int32 NumWindowFrames;
		int32 NumWindowSamples;
		float MonoScaling;

		FAlignedFloatBuffer HopBuffer;
		FAlignedFloatBuffer ChannelBuffer;
		FAlignedFloatBuffer MonoBuffer;

		TUniquePtr<TSlidingBuffer<float> > SlidingBuffer;

		TArray< TUniquePtr<FOnsetStrengthAnalyzer> > OnsetStrengthAnalyzers;

		TArray< TArray<float> > OnsetStrengths;

	};

	/** FOnsetNRTFactory
	 *  
	 *  Factory for creating FOnsetNRT workers and results
	 */
	class FOnsetNRTFactory : public IAnalyzerNRTFactory
	{
	public:

		/** Name of this analyzer type. */
		UE_API virtual FName GetName() const override;

		/** Human readable name of this analyzer. */
		UE_API virtual FString GetTitle() const override;

		/** Create a new FOnsetNRTResult. */
		UE_API virtual TUniquePtr<IAnalyzerNRTResult> NewResult() const override;

		/** Create a new FOnsetNRTWorker 
		 *
		 *  InSettings must be a pointer to FOnsetNRTSetting
		 */
		UE_API virtual TUniquePtr<IAnalyzerNRTWorker> NewWorker(const FAnalyzerNRTParameters& InParams, const IAnalyzerNRTSettings* InSettings) const override;
	};
}

#undef UE_API
