// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"

#define UE_API AUDIOSYNESTHESIACORE_API

namespace Audio
{
	struct FPeakPickerSettings
	{
		int32 NumPreMax = 1440;
		int32 NumPostMax = 1;
		int32 NumPreMean = 4800;
		int32 NumPostMean = 4801;
		int32 NumWait = 1440;
		float MeanDelta = 0.07f;
	};

	class FPeakPicker
	{
		public:
			UE_API FPeakPicker(const FPeakPickerSettings& InSettings);

			UE_API void PickPeaks(TArrayView<const float> InView, TArray<int32>& OutPeakIndices);

		private:
			FPeakPickerSettings Settings;


	};
}

#undef UE_API
