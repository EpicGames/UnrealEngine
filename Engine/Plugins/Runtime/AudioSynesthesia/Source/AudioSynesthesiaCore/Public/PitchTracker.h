// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/AlignedBuffer.h"

namespace Audio
{
	/** Contains information about a pitch observation. */
	struct FPitchInfo
	{
		/** The frequency (in Hz) of the observed pitch. */
		float Frequency = 0.f;

		/** The strength of the pitch.  The exact interpretation of this value will be
		 * dependent upon the specific pitch detector. */
		float Strength = 0.f;

		/** The timestamp (in seconds) of the pitch observation. */
		double Timestamp = 0.f;
	};

	/** Pitch Detector Interface 
	 *
	 * A pitch detector identifies pitch information from a stream of input audio.  The pitch detector should
	 * support multiple calls to `DetectPitches(...)` originating from the same audio source. 
	 */
	class IPitchDetector
	{
		public:
			virtual ~IPitchDetector() = default;

			/** Detect pitches from stream of audio. 
			 *
			 * @param InMonoAudio - The most recent input buffer from a stream of audio.
			 * @param OutPitches - The pitches detected during this call to `DetectPitches` are appended to this array. 
			 */
			virtual void DetectPitches(const FAlignedFloatBuffer& InMonoAudio, TArray<FPitchInfo>& OutPitches) = 0;


			/** Mark the end of an audio stream. 
			 *
			 * Pitch detectors which need to know when an audio stream is finished can overload this method
			 * to handle the end of an audio stream.
			 *
			 * @param OutPitches - The pitches detected during this call to `Finalize` are appended to this array. 
			 */
			virtual void Finalize(TArray<FPitchInfo>& OutPitches) = 0;
	};

	/** Contains information about a pitch track observation. */
	struct FPitchTrackInfo
	{
		/** The array of observations associated with the pitch track. */
		TArray<FPitchInfo> Observations;
	};

	/** Pitch Tracker Interface.
	 *
	 * A pitch tracker idenities multiple pitch tracks from a stream of input audio. The pitch tracker
	 * should support multiple calls to `TrackPitches(...)` originating from the same audio source. 
	 */
	class IPitchTracker
	{
		public:
			virtual ~IPitchTracker() = default;

			/** Detect pitche tracks from stream of audio. 
			 *
			 * @param InMonoAudio - The most recent input buffer from a stream of audio.
			 * @param OutPitchTracks - The pitch tracks detected during this call to `TrackPitches` are appended to this array. 
			 */
			virtual void TrackPitches(const FAlignedFloatBuffer& InMonoAudio, TArray<FPitchTrackInfo>& OutPitchTracks) = 0;

			/** Mark the end of an audio stream. 
			 *
			 * Pitch trackers which need to know when an audio stream is finished can overload this method
			 * to handle the end of an audio stream.
			 *
			 * @param OutPitchTracks - The pitch tracks detected during this call to `Finalize` are added to this array. 
			 */
			virtual void Finalize(TArray<FPitchTrackInfo>& OutPitchTracks) = 0;
	};
}
