// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SynesthesiaSpectrumAnalyzer.h"
#include "IAudioAnalyzerInterface.h"

namespace Audio { template <typename InSampleType> class TSlidingBuffer; }

namespace Audio
{
	/**
	 * Contains settings for Spectrum analyzer.
	 */
	class FSynesthesiaSpectrumAnalysisSettings : public IAnalyzerSettings, public FSynesthesiaSpectrumAnalyzerSettings
	{	
	public:
		/** Number of seconds between spectrum measurements */
		float AnalysisPeriod = 0.01f;
	};

	/**
	 * Holds the spectrum results per a time step for each channel
	 */
	struct FSynesthesiaSpectrumEntry
	{
		int32 Channel = 0;
		float Timestamp = 0.0f;
		TArray<float> SpectrumValues;
	};

	/** 
	 * FSynesthesiaSpectrumResult contains the temporal evolution of loudness for the last buffer of audio analyzed.
	 */
	class FSynesthesiaSpectrumResult : public IAnalyzerResult
	{
	public:
		FSynesthesiaSpectrumResult() {}

		/** Appends an FSynesthesiaSpectrumEntry to the container. */
		AUDIOSYNESTHESIACORE_API void Add(FSynesthesiaSpectrumEntry&& InEntry);

		/** Returns const reference to FSynesthesiaSpectrumEntry array for individual channel. */
		AUDIOSYNESTHESIACORE_API const TArray<FSynesthesiaSpectrumEntry>& GetChannelSpectrumArray(int32 ChannelIdx) const;

		/** Returns the number of channels. */
		AUDIOSYNESTHESIACORE_API int32 GetNumChannels() const;

	private:
		float DurationInSeconds = 0.0f;
		TMap<int32, TArray<FSynesthesiaSpectrumEntry>> ChannelSpectrumArrays;
	};

	/**
	 * FSynesthesiaSpectrumWorker performs Spectrum analysis on input sample buffers.
	 */
	class FSynesthesiaSpectrumAnalysisWorker : public IAnalyzerWorker
	{
	public:
		/** Construct a worker */
		AUDIOSYNESTHESIACORE_API FSynesthesiaSpectrumAnalysisWorker(const FAnalyzerParameters& InParams, const FSynesthesiaSpectrumAnalysisSettings& InAnalyzerSettings);

		/** Analyzes input sample buffer and updates result. */
		AUDIOSYNESTHESIACORE_API virtual void Analyze(TArrayView<const float> InAudio, IAnalyzerResult* OutResult) override;

	private:

		int32 NumChannels = 0;
		bool bDownmixToMono = true;
		int32 NumOutputFrames = 0;
		int32 NumWindowFrames = 0;
		int32 NumWindowSamples = 0;
		int32 NumHopFrames = 0;
		int32 SampleRate = 0;
		int64 FrameCounter = 0;

		FAlignedFloatBuffer MonoBuffer; 
		FAlignedFloatBuffer ChannelBuffer;

		TArray<float> InternalWindow;
		TUniquePtr<TSlidingBuffer<float>> InternalBuffer;
		TUniquePtr<FSynesthesiaSpectrumAnalyzer> SpectrumAnalyzer;
	};

	/**
	 * Defines the Spectrum analyzer and creates related classes.
	 */
	class FSynesthesiaSpectrumAnalysisFactory : public IAnalyzerFactory
	{
	public:

		/** Name of specific analyzer type. */
		AUDIOSYNESTHESIACORE_API virtual FName GetName() const override;

		/** Human readable name of analyzer. */
		AUDIOSYNESTHESIACORE_API virtual FString GetTitle() const override;

		/** Creates a new FSynesthesiaSpectrumResult */
		AUDIOSYNESTHESIACORE_API virtual TUniquePtr<IAnalyzerResult> NewResult() const override;

		/**  Creates a new FSynesthesiaSpectrumAnalysisWorker. This expects IAnalyzerNRTSettings to be a valid pointer to a FSynesthesiaSpectrumAnalysisSettings object. */
		AUDIOSYNESTHESIACORE_API virtual TUniquePtr<IAnalyzerWorker> NewWorker(const FAnalyzerParameters& InParams, const IAnalyzerSettings* InSettings) const override;
	};
}
