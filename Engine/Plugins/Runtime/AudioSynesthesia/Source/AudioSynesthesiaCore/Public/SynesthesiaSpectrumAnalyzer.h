// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/AudioFFT.h"

#define UE_API AUDIOSYNESTHESIACORE_API

namespace Audio { class IFFTAlgorithm; }

namespace Audio
{
	enum class ESynesthesiaSpectrumType : uint8
	{
		// Spectrum frequency values are equal to magnitude of frequency.
		MagnitudeSpectrum,

		// Spectrum frequency values are equal to magnitude squared.
		PowerSpectrum,

		// Returns decibels with a noise floor of -90
		Decibel,
	};

	struct FSynesthesiaSpectrumAnalyzerSettings 
	{	
		int32 FFTSize = 512;

		ESynesthesiaSpectrumType SpectrumType = ESynesthesiaSpectrumType::PowerSpectrum;
		
		EWindowType WindowType = EWindowType::None;
		
		bool bDownmixToMono = true;
	};

	class FSynesthesiaSpectrumAnalyzer
	{
	public:

		/** Construct analyzer */
		UE_API FSynesthesiaSpectrumAnalyzer(float InSampleRate, const FSynesthesiaSpectrumAnalyzerSettings& InSettings);

		/**
		 * Calculate the spectrum values for the input samples. OutSpectrum should have (num input samples / 2) + 1 values. 
		 */
		UE_API void ProcessAudio(TArrayView<const float> InSampleView, TArrayView<float> OutSpectrum);

		/**
		 * Return const reference to settings used inside this analyzer.
		 */
		UE_API const FSynesthesiaSpectrumAnalyzerSettings& GetSettings() const;

	protected:
		FSynesthesiaSpectrumAnalyzerSettings Settings;
		float SampleRate = 0.0f;
		float FFTScaling = 0.0f;

		FWindow Window;
		Audio::FAlignedFloatBuffer WindowedBuffer;
		Audio::FAlignedFloatBuffer FFTOutput;
		TUniquePtr<IFFTAlgorithm> FFT;
	};
}

#undef UE_API
