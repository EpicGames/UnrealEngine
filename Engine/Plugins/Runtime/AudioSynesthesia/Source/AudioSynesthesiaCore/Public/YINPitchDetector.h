// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PitchTracker.h"
#include "DSP/SlidingWindow.h"

#define UE_API AUDIOSYNESTHESIACORE_API

namespace Audio { class FBlockCorrelator; }

namespace Audio
{
	/** Settings for creating a YIN pitch detector. */
	struct FYINPitchDetectorSettings
	{
		/** Time (in seconds) between analysis windows. */
		float AnalysisHopSeconds = 0.01f;

		/** Minimum detectable frequency (Hz). Determines window size — lower values require larger windows. */
		float MinimumFrequency = 80.f;

		/** Maximum detectable frequency (Hz). */
		float MaximumFrequency = 2000.f;

		/** Voiced/unvoiced threshold applied to the cumulative mean normalized difference function.
		 * Lower values are stricter (fewer detections); higher values are more permissive.
		 * Typical range: 0.05 – 0.25. Default (0.15) is appropriate for clean vocal input. */
		float Threshold = 0.15f;
	};

	/** Pitch detector based on the YIN algorithm (de Cheveigné & Kawahara, 2002).
	 *
	 * YIN computes a cumulative mean normalized difference function (CMND) over each analysis
	 * window and finds the first lag below a voiced/unvoiced threshold. Compared to plain
	 * autocorrelation (FAutoCorrelationPitchDetector), YIN strongly suppresses sub-harmonic
	 * octave errors and provides a natural voiced/unvoiced decision via the threshold.
	 *
	 * Strength is reported as (1 - CMND(τ)), where τ is the chosen lag. A value near 1.0
	 * indicates high periodicity; values near 0.0 indicate noise or an unvoiced frame.
	 *
	 * Only one pitch is emitted per analysis window (the fundamental), making this suitable
	 * for monophonic sources such as a singing voice in a karaoke context.
	 */
	class FYINPitchDetector : public IPitchDetector
	{
		public:
			/** Create a YIN pitch detector with the given settings and sample rate. */
			UE_API FYINPitchDetector(const FYINPitchDetectorSettings& InSettings, float InSampleRate);

			UE_API virtual ~FYINPitchDetector();

			/** Detect pitches in the audio. May be called repeatedly with consecutive buffers.
			 *
			 * @param InMonoAudio - A mono audio buffer of any length.
			 * @param OutPitches  - Detected pitch observations are appended to this array.
			 *                      At most one entry is appended per analysis window.
			 */
			UE_API virtual void DetectPitches(const FAlignedFloatBuffer& InMonoAudio, TArray<FPitchInfo>& OutPitches) override;

			/** Resets internal buffers. No additional pitches are emitted on Finalize. */
			UE_API virtual void Finalize(TArray<FPitchInfo>& OutPitches) override;

		private:

			/** Compute the difference function d(τ) = m₁ + m₂(τ) - 2r(τ) via FFT-based cross-correlation.
			 *  r(τ) = Σ_{t=0}^{W/2-1} x[t]·x[t+τ]  is computed with FBlockCorrelator (O(N log N)).
			 *  m₁ = r(0), and m₂(τ) is updated with a O(1) sliding sum of squares per lag. */
			void ComputeDifferenceFunction(const FAlignedFloatBuffer& InWindow);

			/** Normalize d(τ) in-place to the cumulative mean normalized difference d'(τ). */
			void NormalizeDifferenceFunction();

			/** Find the best lag via threshold + parabolic refinement.
			 * @return true if a voiced lag was found, writing its sub-sample value to OutLag. */
			bool FindFundamentalLag(float& OutLag, float& OutStrength) const;

			FYINPitchDetectorSettings Settings;

			float SampleRate;
			int32 WindowNumSamples;
			int32 HalfWindowNumSamples;
			int32 MinLagBin;
			int32 MaxLagBin;
			int64 WindowCounter;

			TSlidingBuffer<float> SlidingBuffer;
			FAlignedFloatBuffer WindowBuffer;

			/** Difference function buffer d(τ), length == HalfWindowNumSamples + 1. */
			TArray<float> DiffBuffer;

			/** FFT-based cross-correlator. Input size = WindowNumSamples, no window, no normalization. */
			TUniquePtr<FBlockCorrelator> Correlator;

			/** Cross-correlation output buffer. Positive lags r(τ) are in [0..HalfWindowNumSamples]. */
			FAlignedFloatBuffer AutoCorrBuffer;

			/** First half of the analysis window zero-padded to WindowNumSamples, reused each frame. */
			FAlignedFloatBuffer FirstHalfBuffer;
	};
}

#undef UE_API
