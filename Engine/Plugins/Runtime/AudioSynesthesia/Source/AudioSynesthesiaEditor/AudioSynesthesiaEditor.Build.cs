// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioSynesthesiaEditor : ModuleRules
{
	public AudioSynesthesiaEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AudioAnalyzer",
				"AudioEditor",
				"AudioSynesthesia",
				"Core",
				"CoreUObject",
				"Engine",
				"EditorFramework",
				"EditorStyle",
				"InputCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
            }
        );

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetDefinition"
			});
	}
}
