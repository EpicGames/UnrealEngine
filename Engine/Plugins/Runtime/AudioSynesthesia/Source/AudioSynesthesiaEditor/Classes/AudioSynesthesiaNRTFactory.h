// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "AudioSynesthesiaNRTFactory.generated.h"

class UAudioSynesthesiaNRT;

/** UAudioSynesthesiaNRTFactory
 *
 * UAudioSynesthesiaNRTFactory implements the factory for UAudioSynesthesiaNRT assets.
 */
UCLASS(MinimalAPI, hidecategories=Object)
class UAudioSynesthesiaNRTFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	/** The type of audio analyzer settings that will be created */
	UPROPERTY(EditAnywhere, Category = AudioSynesthesiaFactory)
	TSubclassOf<UAudioSynesthesiaNRT> AudioSynesthesiaNRTClass;

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};
