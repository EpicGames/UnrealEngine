// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "AudioSynesthesiaNRTSettingsFactory.generated.h"

class UAudioSynesthesiaNRTSettings;

/** UAudioSynesthesiaNRTSettingsFactory
 *
 * UAudioSynesthesiaNRTSettingsFactory implements the factory for UAudioSynesthesiaNRTSettings assets.
 */
UCLASS(MinimalAPI, hidecategories=Object)
class UAudioSynesthesiaNRTSettingsFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	/** The type of audio analyzer settings that will be created */
	UPROPERTY(EditAnywhere, Category = AudioSynesthesiaFactory)
	TSubclassOf<UAudioSynesthesiaNRTSettings> AudioSynesthesiaNRTSettingsClass;

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class,UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};
