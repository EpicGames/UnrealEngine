// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_AudioSynesthesiaDynamic.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_AudioSynesthesiaDynamic"

FText UAssetDefinition_AudioSynesthesiaDynamic::GetAssetDisplayName() const
{
	if (SynesthesiaCDO)
	{
		return SynesthesiaCDO->GetAssetActionName();
	}
	return FText::FromString(TEXT("Audio Analyzer Asset Base"));
}

FLinearColor UAssetDefinition_AudioSynesthesiaDynamic::GetAssetColor() const
{
	if (SynesthesiaCDO)
	{
		return SynesthesiaCDO->GetTypeColor();
	}
	return FColor::White;
}

TSoftClassPtr<UObject> UAssetDefinition_AudioSynesthesiaDynamic::GetAssetClass() const
{
	if (SynesthesiaCDO)
	{
		if (UClass* SupportedClass = SynesthesiaCDO->GetSupportedClass())
		{
			return SupportedClass;
		}

		return SynesthesiaCDO->GetClass();
	}

	return UAudioAnalyzerAssetBase::StaticClass();
}

void UAssetDefinition_AudioSynesthesiaDynamic::Initialize(TSubclassOf<UAudioAnalyzerAssetBase> InClass)
{
	SynesthesiaCDO = TStrongObjectPtr(InClass->GetDefaultObject<UAudioAnalyzerAssetBase>());
}

#undef LOCTEXT_NAMESPACE
