// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AudioAnalyzerAsset.h"
#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "Templates/SubclassOf.h"
#include "AssetDefinition_AudioSynesthesiaDynamic.generated.h"

UCLASS()
class UAssetDefinition_AudioSynesthesiaDynamic : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual bool CanRegisterStatically() const override { return false; }
	// UAssetDefinition End

	void Initialize(TSubclassOf<UAudioAnalyzerAssetBase> InClass);

private:
	TStrongObjectPtr<UAudioAnalyzerAssetBase> SynesthesiaCDO;
};
