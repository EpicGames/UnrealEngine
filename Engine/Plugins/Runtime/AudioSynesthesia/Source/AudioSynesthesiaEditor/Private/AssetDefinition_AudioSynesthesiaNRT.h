// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "AudioSynesthesiaNRT.h"
#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "AssetDefinition_AudioSynesthesiaNRT.generated.h"

UCLASS()
class UAssetDefinition_AudioSynesthesiaNRT : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_AudioSynesthesiaNRT", "Synesthesia NRT"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(200.0f, 150.0f, 200.0f); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAudioSynesthesiaNRT::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Audio,
					NSLOCTEXT("AssetDefinition", "AssetDefinition_AudioSynesthesiaNRTSubMenu", "Advanced"),
					FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_AudioSynesthesiaNRTSubMenuSection", "Analysis"), ECategoryMenuType::Section))
			};
		return Categories;
	}
	// UAssetDefinition End
};
