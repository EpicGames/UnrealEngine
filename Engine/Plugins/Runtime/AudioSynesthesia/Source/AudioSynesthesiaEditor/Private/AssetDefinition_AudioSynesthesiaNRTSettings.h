// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "AudioSynesthesiaNRT.h"
#include "AssetDefinition_AudioSynesthesiaNRTSettings.generated.h"

UCLASS()
class UAssetDefinition_AudioSynesthesiaNRTSettings : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_AudioSynesthesiaNRTSettings", "Synesthesia NRT Settings"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(100.0f, 100.0f, 100.0f); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAudioSynesthesiaNRTSettings::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_AudioSynesthesiaNRTSettingsSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_AudioSynesthesiaNRTSettingsSubMenuSection", "Analysis"), ECategoryMenuType::Section))
		};
		return Categories;
	}
	// UAssetDefinition End
};
