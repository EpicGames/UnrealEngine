// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetTypeActions/AssetDefinitionDefault_AudioDiffable.h"
#include "AudioSynesthesia.h"
#include "AssetDefinition_AudioSynesthesiaSettings.generated.h"

UCLASS()
class UAssetDefinition_AudioSynesthesiaSettings : public UAssetDefinitionDefault_AudioDiffable
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_AudioSynesthesiaSettings", "Synesthesia Settings"); }
	virtual FLinearColor GetAssetColor() const override { return FColor(100.0f, 50.0f, 100.0f); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAudioSynesthesiaSettings::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_AudioSynesthesiaSettingsSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_AudioSynesthesiaSettingsSubMenuSection", "Analysis"), ECategoryMenuType::Section))
		};
		return Categories;
	}
	// UAssetDefinition End
};
