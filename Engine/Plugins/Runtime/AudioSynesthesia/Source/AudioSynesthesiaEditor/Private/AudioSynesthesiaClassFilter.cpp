// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSynesthesiaClassFilter.h"
#include "UObject/Class.h"

namespace Audio
{
	bool FAssetClassParentFilter::IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) 
	{
		return !InClass->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
	}

	bool FAssetClassParentFilter::IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) 
	{
		return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}
}
