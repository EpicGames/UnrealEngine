// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSynesthesiaEditorModule.h"

#include "AssetDefinitionRegistry.h"
#include "AssetDefinition_AudioSynesthesiaDynamic.h"
#include "AudioAnalyzerModule.h"
#include "AudioSynesthesia.h"
#include "AudioSynesthesiaNRT.h"
#include "UObject/UObjectIterator.h"


DEFINE_LOG_CATEGORY(LogAudioSynesthesiaEditor);

class FAudioSynesthesiaEditorModule : public IAudioSynesthesiaEditorModule
{
public:
	FAudioSynesthesiaEditorModule()
	{}

	virtual void StartupModule() override
	{
		AUDIO_ANALYSIS_LLM_SCOPE
		RegisterAssetActions();
	}

	virtual void ShutdownModule() override
	{}


	virtual void RegisterAssetActions() override
	{
		RegisterAudioSynesthesiaChildAssetDefinitionFor<UAudioSynesthesiaNRT>();
		RegisterAudioSynesthesiaChildAssetDefinitionFor<UAudioSynesthesiaNRTSettings>();
		RegisterAudioSynesthesiaChildAssetDefinitionFor<UAudioSynesthesiaSettings>();
	}

private:

	template<class SynesthesiaAssetType>
	void RegisterAudioSynesthesiaChildAssetDefinitionFor() 
	{
		// Look for any sound effect presets to register
		for (TObjectIterator<UClass> It; It; ++It)
		{
			UClass* ChildClass = *It;
			if (ChildClass->HasAnyClassFlags(CLASS_Abstract))
			{
				continue;
			}

			// look for Synesthesia classes 
			UClass* ParentClass = ChildClass->GetSuperClass();
			if (ParentClass && ParentClass->IsChildOf(SynesthesiaAssetType::StaticClass()))
			{
				SynesthesiaAssetType* Synesthesia = ChildClass->GetDefaultObject<SynesthesiaAssetType>();
				check(Synesthesia);

				if (!RegisteredActions.Contains(Synesthesia) && Synesthesia->HasAssetActions())
				{
					RegisteredActions.Add(Synesthesia);
					UAssetDefinition_AudioSynesthesiaDynamic* Definition = NewObject<UAssetDefinition_AudioSynesthesiaDynamic>(
						GetTransientPackage(),
						MakeUniqueObjectName(GetTransientPackage(), UAssetDefinition_AudioSynesthesiaDynamic::StaticClass())
					);
					Definition->Initialize(TSubclassOf<UAudioAnalyzerAssetBase>(ChildClass));
					UAssetDefinitionRegistry::Get()->RegisterAssetDefinition(Definition);
				}
			}
		}
	}	
	TSet<UObject*> RegisteredActions;
};

IMPLEMENT_MODULE( FAudioSynesthesiaEditorModule, AudioSynesthesiaEditor );
