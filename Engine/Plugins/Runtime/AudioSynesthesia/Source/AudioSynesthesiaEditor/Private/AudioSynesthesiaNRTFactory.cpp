// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSynesthesiaNRTFactory.h"

#include "AudioAnalytics.h"
#include "AudioSynesthesiaNRT.h"
#include "Kismet2/SClassPickerDialog.h"
#include "AudioSynesthesiaClassFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioSynesthesiaNRTFactory)

#define LOCTEXT_NAMESPACE "AudioSynesthesiaEditor"

UAudioSynesthesiaNRTFactory::UAudioSynesthesiaNRTFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UAudioSynesthesiaNRT::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
	AudioSynesthesiaNRTClass = nullptr;
}

bool UAudioSynesthesiaNRTFactory::ConfigureProperties()
{
	AudioSynesthesiaNRTClass = nullptr;

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;

	TSharedPtr<Audio::FAssetClassParentFilter> Filter = MakeShareable(new Audio::FAssetClassParentFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());

	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists;
	Filter->AllowedChildrenOfClasses.Add(UAudioSynesthesiaNRT::StaticClass());

	const FText TitleText = LOCTEXT("CreateAudioSynesthesiaNRTOptions", "Pick Synesthesia Class");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UAudioSynesthesiaNRT::StaticClass());

	if (bPressedOk)
	{
		AudioSynesthesiaNRTClass = ChosenClass;
	}

	return bPressedOk;
}

UObject* UAudioSynesthesiaNRTFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UAudioSynesthesiaNRT* NewAudioSynesthesiaNRT = nullptr;
	if (AudioSynesthesiaNRTClass != nullptr)
	{
		NewAudioSynesthesiaNRT = NewObject<UAudioSynesthesiaNRT>(InParent, AudioSynesthesiaNRTClass, InName, Flags);

		Audio::Analytics::RecordEvent_Usage(TEXT("AudioSynesthesia.NRTCreated"));
	}
	return NewAudioSynesthesiaNRT;
}

#undef LOCTEXT_NAMESPACE

