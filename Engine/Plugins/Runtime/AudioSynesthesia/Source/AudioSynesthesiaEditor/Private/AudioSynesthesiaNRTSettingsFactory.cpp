// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSynesthesiaNRTSettingsFactory.h"

#include "AudioAnalytics.h"
#include "AudioSynesthesiaNRT.h"
#include "Kismet2/SClassPickerDialog.h"
#include "AudioSynesthesiaClassFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioSynesthesiaNRTSettingsFactory)

UAudioSynesthesiaNRTSettingsFactory::UAudioSynesthesiaNRTSettingsFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UAudioSynesthesiaNRTSettings::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
	AudioSynesthesiaNRTSettingsClass = nullptr;
}
#define LOCTEXT_NAMESPACE "AudioSynesthesiaEditor"

bool UAudioSynesthesiaNRTSettingsFactory::ConfigureProperties()
{
	AudioSynesthesiaNRTSettingsClass = nullptr;

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;

	TSharedPtr<Audio::FAssetClassParentFilter> Filter = MakeShareable(new Audio::FAssetClassParentFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());

	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists;
	Filter->AllowedChildrenOfClasses.Add(UAudioSynesthesiaNRTSettings::StaticClass());

	const FText TitleText = LOCTEXT("CreateAudioSynesthesiaNRTSettingOptions", "Pick Synesthesia Settings Class");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UAudioSynesthesiaNRTSettings::StaticClass());

	if (bPressedOk)
	{
		AudioSynesthesiaNRTSettingsClass = ChosenClass;
	}

	return bPressedOk;
}

UObject* UAudioSynesthesiaNRTSettingsFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UAudioSynesthesiaNRTSettings* NewAudioSynesthesiaNRTSettings = nullptr;
	if (AudioSynesthesiaNRTSettingsClass != nullptr)
	{
		NewAudioSynesthesiaNRTSettings = NewObject<UAudioSynesthesiaNRTSettings>(InParent, AudioSynesthesiaNRTSettingsClass, InName, Flags);
		Audio::Analytics::RecordEvent_Usage(TEXT("AudioSynesthesia.NRTSettingsCreated"));
	}
	return NewAudioSynesthesiaNRTSettings;
}

#undef LOCTEXT_NAMESPACE

