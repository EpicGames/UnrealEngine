// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSynesthesiaSettingsFactory.h"

#include "AudioAnalytics.h"
#include "AudioSynesthesia.h"
#include "Kismet2/SClassPickerDialog.h"
#include "AudioSynesthesiaClassFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioSynesthesiaSettingsFactory)

UAudioSynesthesiaSettingsFactory::UAudioSynesthesiaSettingsFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UAudioSynesthesiaSettings::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
	AudioSynesthesiaSettingsClass = nullptr;
}
#define LOCTEXT_NAMESPACE "AudioSynesthesiaEditor"

bool UAudioSynesthesiaSettingsFactory::ConfigureProperties()
{
	AudioSynesthesiaSettingsClass = nullptr;

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;

	TSharedPtr<Audio::FAssetClassParentFilter> Filter = MakeShareable(new Audio::FAssetClassParentFilter);
	Options.ClassFilters.Add(Filter.ToSharedRef());

	Filter->DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists;
	Filter->AllowedChildrenOfClasses.Add(UAudioSynesthesiaSettings::StaticClass());

	const FText TitleText = LOCTEXT("CreateAudioSynesthesiaSettingOptions", "Pick Synesthesia Settings Class");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UAudioSynesthesiaSettings::StaticClass());

	if (bPressedOk)
	{
		AudioSynesthesiaSettingsClass = ChosenClass;
	}

	return bPressedOk;
}

UObject* UAudioSynesthesiaSettingsFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UAudioSynesthesiaSettings* NewAudioSynesthesiaSettings = nullptr;
	if (AudioSynesthesiaSettingsClass != nullptr)
	{
		NewAudioSynesthesiaSettings = NewObject<UAudioSynesthesiaSettings>(InParent, AudioSynesthesiaSettingsClass, InName, Flags);

		Audio::Analytics::RecordEvent_Usage(TEXT("AudioSynesthesia.SettingsCreated"));
	}
	return NewAudioSynesthesiaSettings;
}

#undef LOCTEXT_NAMESPACE

