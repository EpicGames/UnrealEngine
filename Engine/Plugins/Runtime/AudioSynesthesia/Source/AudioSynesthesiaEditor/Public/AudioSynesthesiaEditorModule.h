// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

AUDIOSYNESTHESIAEDITOR_API DECLARE_LOG_CATEGORY_EXTERN(LogAudioSynesthesiaEditor, Log, All);

extern const FName AudioSynesthesiaEditorAppIdentifier;

/** audio analyzer editor module interface */
class IAudioSynesthesiaEditorModule :	public IModuleInterface
{
public:
	/** Registers audio analyzer asset actions. */
	virtual void RegisterAssetActions() = 0;
};
